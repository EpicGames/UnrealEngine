{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.0.1",
	"FriendlyName": "ML Adapter",
	"Description": "A framework for training and utilizing machine learning agents in games. Creates an RPC interface through which an external process can query game state and control in-game actors. Once trained, agents can be run in-engine via neural networks loaded from ONNX models.",
	"Category": "AI",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MLAdapter",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "MLAdapterTestSuite",
			"Type": "DeveloperTool",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "GameplayAbilities",
			"Enabled": true
		},
		{
			"Name": "EnhancedInput",
			"Enabled": true
		}
	]
}