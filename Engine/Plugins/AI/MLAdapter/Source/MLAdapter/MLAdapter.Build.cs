// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MLAdapter : ModuleRules
    {
        public MLAdapter(ReadOnlyTargetRules Target) : base(Target)
        {
			// recheck this if rpclib is updated
			StaticAnalyzerDisabledCheckers.Add("unix.Malloc");

			// rcplib is using exceptions so we have to enable that
			bEnableExceptions = true;
            PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

            PublicDependencyModuleNames.AddRange(
                new string[] {
                    "Core",
                    "CoreUObject",
                    "Engine",
					"EnhancedInput",
                    "GameplayTags",
                    "AIModule",
                    "InputCore",
                    "Json",
                    "JsonUtilities",
                    "GameplayAbilities",
					"NNE"
				}
            );

            PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "DeveloperSettings"
                }
            );

            // RPCLib disabled on other platforms
            if (Target.Platform == UnrealTargetPlatform.Win64 ||
				Target.Platform == UnrealTargetPlatform.Mac ||
				Target.Platform == UnrealTargetPlatform.Linux)
            {
                PublicDefinitions.Add("WITH_RPCLIB=1");
				PrivateDependencyModuleNames.Add("RPCLib");

				if (Target.Platform == UnrealTargetPlatform.Win64 && Target.Architecture == UnrealArch.Arm64)
				{
					PrivateDefinitions.Add("__arm64"); // expected by rpclib headers
				}
			}
            else
            {
                PublicDefinitions.Add("WITH_RPCLIB=0");
            }

            if (Target.bBuildEditor == true)
            {
                PrivateDependencyModuleNames.Add("EditorFramework");
                PrivateDependencyModuleNames.Add("UnrealEd");
            }

            SetupGameplayDebuggerSupport(Target);
        }
    }
}
