// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actuators/MLAdapterActuator.h"
#include "MLAdapterTypes.h"
#include "Managers/MLAdapterManager.h"


namespace
{
	uint32 NextActuatorID = FMLAdapter::InvalidActuatorID + 1;
}

UMLAdapterActuator::UMLAdapterActuator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ElementID = FMLAdapter::InvalidActuatorID;
}

void UMLAdapterActuator::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		if (GetClass()->HasAnyClassFlags(CLASS_Abstract) == false)
		{
			ElementID = NextActuatorID++;
		}
	}
	else
	{
		const UMLAdapterActuator* CDO = GetDefault<UMLAdapterActuator>(GetClass());
		// already checked 
		ElementID = CDO->ElementID;
	}
}
