// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actuators/MLAdapterActuator_Camera.h"
#include "HAL/IConsoleManager.h"
#include "MLAdapterTypes.h"
#include "MLAdapterSpace.h"
#include "MLAdapterInputHelper.h"
#include "Agents/MLAdapterAgent.h"
#include "GameFramework/PlayerController.h"
#include "Debug/DebugHelpers.h"


namespace FMLAdapterActuatorCameraTweakables
{
	int32 bSkipActing = 0;
}

namespace
{
	FAutoConsoleVariableRef CVar_SkipActing(TEXT("mladapter.actuator.camera.skip_acting"), FMLAdapterActuatorCameraTweakables::bSkipActing, TEXT("Whether the actuator should stop affecting the camera"), ECVF_Default);
}

UMLAdapterActuator_Camera::UMLAdapterActuator_Camera(const FObjectInitializer& ObjectInitializer)
{
	bVectorMode = true;
	bConsumeData = true;
}

void UMLAdapterActuator_Camera::Configure(const TMap<FName, FString>& Params)
{
	Super::Configure(Params);

	const FName NAME_Mode = TEXT("mode");
	const FString* ModeValue = Params.Find(NAME_Mode);
	if (ModeValue != nullptr)
	{
		bVectorMode = (ModeValue->Find(TEXT("vector")) != INDEX_NONE);
	}

	UpdateSpaceDef();
}

TSharedPtr<FMLAdapter::FSpace> UMLAdapterActuator_Camera::ConstructSpaceDef() const
{
	static const float MaxFPS = 24.f;
	FMLAdapter::FSpace* Result = bVectorMode
		? new FMLAdapter::FSpace_Box({ 3 })
		: new FMLAdapter::FSpace_Box({ 2 }, -360.f * MaxFPS, 360.f * MaxFPS);
	return MakeShareable(Result);
}

void UMLAdapterActuator_Camera::Act(const float DeltaTime)
{
	APlayerController* PC = Cast<APlayerController>(GetAgent().GetAvatar());
	ensure(PC != nullptr || GetAgent().GetAvatar() == nullptr);
	if (PC == nullptr)
	{
		return;
	}

	FRotator Rotation = FRotator::ZeroRotator;
	{
		FScopeLock Lock(&ActionCS);
		Rotation = HeadingRotator;
		if (bVectorMode)
		{
			Rotation = HeadingVector.Rotation();
		}
		if (bConsumeData)
		{
			HeadingRotator = FRotator::ZeroRotator;
			HeadingVector = FVector::ForwardVector;
		}
	}
	
	if (!FMLAdapterActuatorCameraTweakables::bSkipActing)
	{
		PC->AddPitchInput(Rotation.Pitch * DeltaTime);
		PC->AddYawInput(Rotation.Yaw * DeltaTime);
	}
}

void UMLAdapterActuator_Camera::DigestInputData(FMLAdapterMemoryReader& ValueStream)
{
	FScopeLock Lock(&ActionCS);
	if (bVectorMode)
	{
		FVector3f HeadingVector3f;
		ValueStream << HeadingVector3f;
		HeadingVector = FVector(HeadingVector3f);
	}
	else
	{
		float Pitch;
		float Yaw;
		ValueStream << Pitch << Yaw;
		HeadingRotator.Pitch = Pitch;
		HeadingRotator.Yaw = Yaw;
	}
}

#if WITH_GAMEPLAY_DEBUGGER
#include "GameplayDebuggerCategory.h"
void UMLAdapterActuator_Camera::DescribeSelfToGameplayDebugger(FGameplayDebuggerCategory& DebuggerCategory) const
{
	FRotator Rotation = HeadingRotator;
	if (bVectorMode)
	{
		Rotation = HeadingVector.Rotation();
	}

	DebugRuntimeString = FString::Printf(TEXT("[%.2f, %2.f]"), Rotation.Pitch, Rotation.Yaw);
	Super::DescribeSelfToGameplayDebugger(DebuggerCategory);
}
#endif // WITH_GAMEPLAY_DEBUGGER