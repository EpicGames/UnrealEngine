// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actuators/MLAdapterActuator_EnhancedInput.h"
#include "MLAdapterTypes.h"
#include "MLAdapterSpace.h"
#include "MLAdapterInputHelper.h"
#include "Agents/MLAdapterAgent.h"
#include "GameFramework/PlayerController.h"
#include "Debug/DebugHelpers.h"
#include "EnhancedInputSubsystems.h"
#include "Engine/LocalPlayer.h"
#include "EnhancedPlayerInput.h"
#include "MLAdapterInputHelper.h"


void UMLAdapterActuator_EnhancedInput::Configure(const TMap<FName, FString>& Params)
{
	Super::Configure(Params);

	const FName NAME_bClearActionOnUse = TEXT("bClearActionOnUse");
	const FString* bClearActionOnUseValue = Params.Find(NAME_bClearActionOnUse);
	if (bClearActionOnUseValue != nullptr)
	{
		bClearActionOnUse = (*bClearActionOnUseValue != "0");
	}

	UpdateSpaceDef();
}

TSharedPtr<FMLAdapter::FSpace> UMLAdapterActuator_EnhancedInput::ConstructSpaceDef() const
{
	return FMLAdapterInputHelper::ConstructEnhancedInputSpaceDef(TrackedActions);
}

void UMLAdapterActuator_EnhancedInput::Act(const float DeltaTime)
{
	APlayerController* PC = Cast<APlayerController>(GetControllerAvatar());
	if (PC == nullptr)
	{
		return;
	}

	FScopeLock Lock(&ActionCS);

	if (InputData.IsEmpty())
	{
		return;
	}

	checkf(InputData.Num() % SpaceDef->Num() == 0, TEXT("InputData contains an unexpected number of elements"));

	if (UEnhancedInputLocalPlayerSubsystem* Subsystem = ULocalPlayer::GetSubsystem<UEnhancedInputLocalPlayerSubsystem>(PC->GetLocalPlayer()))
	{
		UEnhancedPlayerInput* PlayerInput = Subsystem->GetPlayerInput();

		int32 InputDataIndex = 0;
		while (InputDataIndex != SpaceDef->Num()) // we might have to loop over all the actions multiple times if digesting is running faster than us
		{
			for (const UInputAction* InputAction : TrackedActions)
			{
				switch (InputAction->ValueType)
				{
				case EInputActionValueType::Boolean:
				{
					bool Value = InputData[InputDataIndex] == 1.0f ? true : false;
					FInputActionValue ActionValue(Value);
					PlayerInput->InjectInputForAction(InputAction, ActionValue);
					InputDataIndex += 1;
					break;
				}
				case EInputActionValueType::Axis1D:
				{
					float Value = InputData[InputDataIndex];
					FInputActionValue ActionValue(Value);
					PlayerInput->InjectInputForAction(InputAction, ActionValue);
					InputDataIndex += 1;
					break;
				}
				case EInputActionValueType::Axis2D:
				{
					FVector2D Value(InputData[InputDataIndex], InputData[InputDataIndex + 1]);
					FInputActionValue ActionValue(Value);
					PlayerInput->InjectInputForAction(InputAction, ActionValue);
					InputDataIndex += 2;
					break;
				}
				case EInputActionValueType::Axis3D:
				{
					FVector Value(InputData[InputDataIndex], InputData[InputDataIndex + 1], InputData[InputDataIndex + 2]);
					FInputActionValue ActionValue(Value);
					PlayerInput->InjectInputForAction(InputAction, ActionValue);
					InputDataIndex += 3;
					break;
				}
				default:
					checkf(false, TEXT("Unsupported value type for input action value!"));
					break;
				}
			}
		}
	}
	
	if (bClearActionOnUse)
	{
		InputData.Empty(InputData.Num());
	}
}

void UMLAdapterActuator_EnhancedInput::DigestInputData(FMLAdapterMemoryReader& ValueStream)
{
	FScopeLock Lock(&ActionCS);

	if (bClearActionOnUse)
	{
		const int32 OldSize = InputData.Num();
		InputData.AddUninitialized(SpaceDef->Num());
		// offsetting the serialization since there might be unprocessed data in InputData
		ValueStream.Serialize(InputData.GetData() + OldSize, SpaceDef->Num() * sizeof(float));
	}
	else
	{
		if (InputData.Num() == 0)
		{
			InputData.AddUninitialized(SpaceDef->Num());
		}

		// Overwrite the currently executing actions
		ValueStream.Serialize(InputData.GetData(), SpaceDef->Num() * sizeof(float));
	}
}