// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actuators/MLAdapterActuator_InputKey.h"
#include "MLAdapterTypes.h"
#include "MLAdapterSpace.h"
#include "MLAdapterInputHelper.h"
#include "Agents/MLAdapterAgent.h"
#include "GameFramework/PlayerController.h"
#include "Debug/DebugHelpers.h"


UMLAdapterActuator_InputKey::UMLAdapterActuator_InputKey(const FObjectInitializer& ObjectInitializer)
{
	bIsMultiBinary = false;
}

void UMLAdapterActuator_InputKey::Configure(const TMap<FName, FString>& Params) 
{
	Super::Configure(Params);

	TArray<FName> IgnoreKeys;
	const FName NAME_IgnoreKeys = TEXT("ignore_keys");
	const FString* IgnoreKeysValue = Params.Find(NAME_IgnoreKeys);
	if (IgnoreKeysValue != nullptr)
	{
		TArray<FString> Tokens;
		// contains a list of keys to not press
		IgnoreKeysValue->ParseIntoArrayWS(Tokens, TEXT(","));
		IgnoreKeys.Reserve(Tokens.Num());
		for (const FString& Token : Tokens)
		{
			IgnoreKeys.Add(FName(Token));
		}
	}

	TArray<FName> IgnoreActions;
	const FName NAME_IgnoreActions = TEXT("ignore_actions");
	const FString* IgnoreActionsValue = Params.Find(NAME_IgnoreActions);
	if (IgnoreActionsValue != nullptr)
	{
		TArray<FString> Tokens;
		// contains a list of actions to ignore
		IgnoreActionsValue->ParseIntoArrayWS(Tokens, TEXT(","));
		IgnoreActions.Reserve(Tokens.Num());
		for (const FString& Token : Tokens)
		{
			IgnoreActions.Add(FName(Token));
		}
	}

	TMap<FKey, int32> TmpKeyMap;
	FMLAdapterInputHelper::CreateInputMap(RegisteredKeys, TmpKeyMap);

	RegisteredKeys.RemoveAllSwap([&IgnoreKeys](const TTuple<FKey, FName>& Element) -> bool
		{
			return IgnoreKeys.Find(Element.Key.GetFName()) != INDEX_NONE;
		});
	RegisteredKeys.RemoveAllSwap([&IgnoreActions](const TTuple<FKey, FName>& Element) -> bool
		{
			return IgnoreActions.Find(Element.Value) != INDEX_NONE;
		});

	PressedKeys.Init(false, RegisteredKeys.Num());

	UpdateSpaceDef();
}

TSharedPtr<FMLAdapter::FSpace> UMLAdapterActuator_InputKey::ConstructSpaceDef() const
{
	FMLAdapter::FSpace* Result = nullptr;

	if (bIsMultiBinary)
	{
		NOT_IMPLEMENTED();
		Result = new FMLAdapter::FSpace_Dummy();
	}
	else
	{
		Result = new FMLAdapter::FSpace_Discrete(RegisteredKeys.Num());
	}
	
	return MakeShareable(Result);
}

void UMLAdapterActuator_InputKey::Act(const float DeltaTime)
{
	APlayerController* PC = Cast<APlayerController>(GetControllerAvatar());
	if (PC == nullptr)
	{
		return;
	}

	FScopeLock Lock(&ActionCS);

	TBitArray<> OldPressedKeys = PressedKeys;
	PressedKeys.Init(false, RegisteredKeys.Num());

	for (int Index = 0; Index < InputData.Num(); ++Index)
	{
		int KeyID = Index % RegisteredKeys.Num();
		if (InputData[Index] != 0.f)
		{
			PressedKeys[KeyID] = true;
			if (OldPressedKeys[KeyID] == false)
			{
				// press only if not pressed previously
				// @todo this should probably be optional
				PC->InputKey(FInputKeyEventArgs::CreateSimulated(RegisteredKeys[KeyID].Get<0>(), IE_Pressed, 1.0f));
			}
		}
	}
	InputData.Empty(InputData.Num());

	for (int Index = 0; Index < RegisteredKeys.Num(); ++Index)
	{
		if (OldPressedKeys[Index] && !PressedKeys[Index])
		{
			PC->InputKey(FInputKeyEventArgs::CreateSimulated(RegisteredKeys[Index].Get<0>(), IE_Released, 1.0f));
		}
	}
}

void UMLAdapterActuator_InputKey::DigestInputData(FMLAdapterMemoryReader& ValueStream)
{
	FScopeLock Lock(&ActionCS);

	const int32 OldSize = InputData.Num();
	InputData.AddUninitialized(RegisteredKeys.Num());
	// offsetting the serialization since there might be unprocessed data in InputData
	ValueStream.Serialize(InputData.GetData() + OldSize, RegisteredKeys.Num() * sizeof(float));
}