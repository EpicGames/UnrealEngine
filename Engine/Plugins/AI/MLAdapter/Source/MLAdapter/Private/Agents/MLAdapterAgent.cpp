// Copyright Epic Games, Inc. All Rights Reserved.

#include "Agents/MLAdapterAgent.h"
#include "Engine/GameInstance.h"
#include "Sensors/MLAdapterSensor.h"
#include "Sessions/MLAdapterSession.h"
#include "MLAdapterSpace.h"
#include "MLAdapterLibrarian.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "Actuators/MLAdapterActuator_InputKey.h"
#include "UObject/UObjectGlobals.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/Pawn.h"
#include "UObject/GarbageCollection.h"
#include "GameFramework/PlayerState.h"


//----------------------------------------------------------------------//
// FMLAdapterAgentHelpers 
//----------------------------------------------------------------------//
namespace FMLAdapterAgentHelpers
{
	template<typename ControllerPtrType, typename PawnPtrType>
	bool GetAsPawnAndController_Internal(AActor* Avatar, ControllerPtrType& OutController, PawnPtrType& OutPawn)
	{
		if (Avatar == nullptr)
		{
			return false;
		}

		APawn* AsPawn = Cast<APawn>(Avatar);
		if (AsPawn)
		{
			OutPawn = AsPawn;
			OutController = AsPawn->GetController();
			return true;
		}

		AController* AsController = Cast<AController>(Avatar);
		if (AsController)
		{
			OutPawn = AsController->GetPawn();
			OutController = AsController;
			return true;
		}

		return false;
	}

	bool GetAsPawnAndController(AActor* Avatar, AController*& OutController, APawn*& OutPawn)
	{
		return GetAsPawnAndController_Internal(Avatar, OutController, OutPawn);
	}

	bool GetAsPawnAndController(AActor* Avatar, TObjectPtr<AController>& OutController, TObjectPtr<APawn>& OutPawn)
	{
		return GetAsPawnAndController_Internal(Avatar, OutController, OutPawn);
	}
}

//----------------------------------------------------------------------//
// FMLAdapterAgentConfig
//----------------------------------------------------------------------//
FMLAdapterParameterMap& FMLAdapterAgentConfig::AddSensor(const FName SensorName, FMLAdapterParameterMap&& Parameters)
{
	FMLAdapterParameterMap& Entry = Sensors.Add(SensorName, Parameters);
	return Entry;
}

FMLAdapterParameterMap& FMLAdapterAgentConfig::AddActuator(const FName ActuatorName, FMLAdapterParameterMap&& Parameters)
{
	FMLAdapterParameterMap& Entry = Actuators.Add(ActuatorName, Parameters);
	return Entry;
}

//----------------------------------------------------------------------//
// UMLAdapterAgent 
//----------------------------------------------------------------------//
UMLAdapterAgent::UMLAdapterAgent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	AgentID = FMLAdapter::InvalidAgentID;
	bEverHadAvatar = false;
	bRegisteredForPawnControllerChange = false;
}

void UMLAdapterAgent::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		Actuators.Sort(FAgentElementSort());
		Sensors.Sort(FAgentElementSort());

		if (AvatarClass != nullptr)
		{
			GetSession().RequestAvatarForAgent(*this);
		}
	}
}

void UMLAdapterAgent::BeginDestroy()
{
	ShutDownSensorsAndActuators();
	// forcing unhooking from all event delegates
	SetAvatar(nullptr); 

	if (bRegisteredForPawnControllerChange)
	{
		UGameInstance* GameInstance = GetSession().GetGameInstance();
		if (GameInstance)
		{
			GameInstance->GetOnPawnControllerChanged().RemoveAll(this);
			bRegisteredForPawnControllerChange = false;
		}
	}

	Super::BeginDestroy();
}

bool UMLAdapterAgent::RegisterSensor(UMLAdapterSensor& Sensor)
{
	ensure(Sensor.IsConfiguredForAgent(*this));
	Sensors.Add(&Sensor);
	return true;
}

void UMLAdapterAgent::OnAvatarDestroyed(AActor* DestroyedActor)
{
	if (DestroyedActor == Avatar)
	{
		SetAvatar(nullptr);

		if (AgentConfig.bAutoRequestNewAvatarUponClearingPrev)
		{
			// note that after this call we might not have the avatar just yet 
			// since the world might not have any. The Session will make sure to 
			// assign us one as soon as one's ready.
			GetSession().RequestAvatarForAgent(*this);
		}
	}
}

void UMLAdapterAgent::OnPawnChanged(APawn* NewPawn, AController* InController)
{
	ensure(Controller == InController);
	if (Controller != InController)
	{
		// this came from a different controller we somehow missed unbinding from. Ignore.
		return;
	}

	if (Pawn == NewPawn)
	{
		return;
	}

	// let every sense that requires a pawn know that the pawn changed 
	for (UMLAdapterSensor* Sensor : Sensors)
	{
		Sensor->OnPawnChanged(Pawn, NewPawn);
	}

	Pawn = NewPawn;
}

void UMLAdapterAgent::OnPawnControllerChanged(APawn* InPawn, AController* InController)
{
	if (InPawn == Pawn)
	{
		if (Pawn == Avatar)
		{
			Controller = InController;
		}
		// if controller is the main avatar we might have just lost our pawn
		else if (Controller && Controller != InController && Controller == Avatar)
		{
			OnPawnChanged(Controller->GetPawn() == InPawn ? nullptr : Controller->GetPawn(), Controller);
		}
	}
}

void UMLAdapterAgent::Sense(const float DeltaTime)
{
	CurrentActionDuration += DeltaTime;

	bool ShouldDecideNow = !bEnableActionDuration || CurrentActionDuration >= ActionDurationSeconds;
	if (ShouldDecideNow)
	{
		FScopeLock Lock(&ActionDurationCS);

		CurrentActionDuration = 0.f;
		bActionDurationElapsed = true;

		for (UMLAdapterSensor* Sensor : Sensors)
		{
			// not that due to the system's design Sensor won't be null
			Sensor->Sense(DeltaTime);
		}
	}
}

void UMLAdapterAgent::Think(const float DeltaTime)
{
	// Can be implemented in child class
}

void UMLAdapterAgent::Act(const float DeltaTime)
{
	for (UMLAdapterActuator* Actuator : Actuators)
	{
		Actuator->Act(DeltaTime);
	}
}

void UMLAdapterAgent::DigestActions(FMLAdapterMemoryReader& ValueStream)
{
	for (UMLAdapterActuator* Actuator : Actuators)
	{
		Actuator->DigestInputData(ValueStream);
	}
}

void UMLAdapterAgent::GetObservations(FMLAdapterMemoryWriter& Ar)
{
	for (UMLAdapterSensor* Sensor : Sensors)
	{
		Sensor->GetObservations(Ar);
	}
}

void UMLAdapterAgent::ShutDownSensorsAndActuators()
{	
	for (UMLAdapterActuator* Actuator : Actuators)
	{
		if (Actuator)
		{
			Actuator->Shutdown();
		}
	}
	Actuators.Reset();

	for (UMLAdapterSensor* Sensor : Sensors)
	{
		if (Sensor)
		{
			Sensor->Shutdown();
		}
	}
	Sensors.Reset();
}

void UMLAdapterAgent::Configure(const FMLAdapterAgentConfig& NewConfig)
{
	ShutDownSensorsAndActuators();

	TSubclassOf<AActor> PreviousAvatarClass = AvatarClass;
	AgentConfig = NewConfig;

	for (const TTuple<FName, FMLAdapterParameterMap>& KeyValue : AgentConfig.Actuators)
	{
		UClass* ResultClass = FMLAdapterLibrarian::Get().FindActuatorClass(KeyValue.Key);
		if (ResultClass)
		{
			UMLAdapterActuator* NewActuator = FMLAdapter::NewObject<UMLAdapterActuator>(this, ResultClass);
			check(NewActuator);
			NewActuator->SetNickname(KeyValue.Key.ToString());
			NewActuator->Configure(KeyValue.Value.Params);
			Actuators.Add(NewActuator);
		}
	}
	Actuators.Sort(FAgentElementSort());

	for (const TTuple<FName, FMLAdapterParameterMap>& KeyValue : AgentConfig.Sensors)
	{
		UClass* ResultClass = FMLAdapterLibrarian::Get().FindSensorClass(KeyValue.Key);
		if (ResultClass)
		{
			UMLAdapterSensor* NewSensor = FMLAdapter::NewObject<UMLAdapterSensor>(this, ResultClass);
			check(NewSensor);
			NewSensor->SetNickname(KeyValue.Key.ToString());
			NewSensor->Configure(KeyValue.Value.Params);
			Sensors.Add(NewSensor);
		}
	}
	Sensors.Sort(FAgentElementSort());

	if (NewConfig.AvatarClassName != NAME_None)
	{
		AvatarClass = UClass::TryFindTypeSlow<UClass>(*NewConfig.AvatarClassName.ToString());
	}

	if (!AvatarClass)
	{
		AvatarClass = AActor::StaticClass();
	}

	ensure(AvatarClass || Avatar);

	if (AvatarClass && (Avatar == nullptr || !IsSuitableAvatar(*Avatar)))
	{
		SetAvatar(nullptr);

		// if avatar class changed make sure the following RequestAvatarForAgent actually tries to find an avatar
		// rather than just ignoring the request due to the agent already being in AwaitingAvatar
		const bool bForceUpdate = (AvatarClass != PreviousAvatarClass);

		// note that after this call we might not have the avatar just yet 
		// since the world might not have any. The Session will make sure to 
		// assign us one as soon as one's ready.
		GetSession().RequestAvatarForAgent(*this, /*World=*/nullptr, bForceUpdate);
	}
	else if (ensure(Avatar))
	{
		// newly created actuators and sensors might need the information about 
		// the current avatar
		for (UMLAdapterSensor* Sensor : Sensors)
		{
			Sensor->OnAvatarSet(Avatar);
		}
		for (UMLAdapterActuator* Actuator : Actuators)
		{
			Actuator->OnAvatarSet(Avatar);
		}
	}
}

UMLAdapterSession& UMLAdapterAgent::GetSession()
{
	UMLAdapterSession* Session = CastChecked<UMLAdapterSession>(GetOuter());
	return *Session;
}

void UMLAdapterAgent::GetActionSpaceDescription(FMLAdapterSpaceDescription& OutSpaceDesc) const
{
	FMLAdapterDescription ElementDesc;

	for (UMLAdapterActuator* Actuator : Actuators)
	{
		if (Actuator)
		{
			ElementDesc.Reset();
			ElementDesc.Add(Actuator->GetSpaceDef());
			OutSpaceDesc.Add(Actuator->GetNickname(), ElementDesc);
		}
	}
}

void UMLAdapterAgent::GetObservationSpaceDescription(FMLAdapterSpaceDescription& OutSpaceDesc) const
{
	FMLAdapterDescription ElementDesc;

	for (UMLAdapterSensor* Sensor : Sensors)
	{
		if (Sensor)
		{
			ElementDesc.Reset();
			ElementDesc.Add(Sensor->GetSpaceDef());
			OutSpaceDesc.Add(Sensor->GetNickname(), ElementDesc);
		}
	}
}

bool UMLAdapterAgent::IsSuitableAvatar(AActor& InAvatar) const
{	
	return AgentConfig.bAvatarClassExact 
		? InAvatar.GetClass() == AvatarClass.Get()
		: InAvatar.IsA(AvatarClass);
}

void UMLAdapterAgent::SetAvatar(AActor* InAvatar)
{
	if (InAvatar == Avatar)
	{
		// do nothing on purpose
		return;
	}

	if (InAvatar != nullptr && IsSuitableAvatar(*InAvatar) == false)
	{
		UE_LOGF(LogMLAdapter, Log, "SetAvatar was called for agent %u but %ls is not a valid avatar (required avatar class %ls)",
			AgentID, *InAvatar->GetName(), *GetNameSafe(AvatarClass));
		return;
	}

	AActor* PrevAvatar = Avatar;
	AController* PrevController = Controller;
	APawn* PrevPawn = Pawn;
	if (Avatar)
	{
		Avatar->OnDestroyed.RemoveAll(this);
		Avatar = nullptr;
	}

	if (InAvatar == nullptr)
	{
		Controller = nullptr;
		Pawn = nullptr;
	}
	else
	{
		bEverHadAvatar = true;
		Avatar = InAvatar;

		Pawn = nullptr;
		Controller = nullptr;
		FMLAdapterAgentHelpers::GetAsPawnAndController(Avatar, Controller, Pawn);

		if (Avatar)
		{
			Avatar->OnDestroyed.AddDynamic(this, &UMLAdapterAgent::OnAvatarDestroyed);
		}
	}

	// actuators and sensors might need the information that the avatar changed
	for (UMLAdapterSensor* Sensor : Sensors)
	{
		Sensor->OnAvatarSet(Avatar);
	}
	for (UMLAdapterActuator* Actuator : Actuators)
	{
		Actuator->OnAvatarSet(Avatar);
	}

	// unregister from unused notifies

	if (Controller != PrevController)
	{
		if (PrevController && PrevController == PrevAvatar)
		{
			PrevController->GetOnNewPawnNotifier().RemoveAll(this);
		}
		// when the controller is the main avatar
		if (Controller != nullptr && (Avatar == Controller))
		{
			Controller->GetOnNewPawnNotifier().AddUObject(this, &UMLAdapterAgent::OnPawnChanged, ToRawPtr(Controller));
		}
	}

	if ((Controller != nullptr || Pawn != nullptr) && bRegisteredForPawnControllerChange == false)
	{
		UGameInstance* GameInstance = GetSession().GetGameInstance();
		if (GameInstance)
		{
			GameInstance->GetOnPawnControllerChanged().AddDynamic(this, &UMLAdapterAgent::OnPawnControllerChanged);
			bRegisteredForPawnControllerChange = true;
		}
	}
}

float UMLAdapterAgent::GetReward() const
{
	if (Avatar == nullptr)
	{
		return 0.f;
	}
	FGCScopeGuard GCScopeGuard;
	AController* AsController = FMLAdapter::ActorToController(*Avatar);
	return AsController && AsController->PlayerState ? AsController->PlayerState->GetScore() : 0.f;
}

bool UMLAdapterAgent::IsDone() const
{
	return AgentConfig.bAutoRequestNewAvatarUponClearingPrev == false 
		&& Avatar == nullptr
		&& bEverHadAvatar == true;
}

void UMLAdapterAgent::EnableActionDuration(bool bEnable, float DurationSeconds)
{
	bEnableActionDuration = bEnable;
	ActionDurationSeconds = DurationSeconds;
}

bool UMLAdapterAgent::TryResetActionDuration()
{
	FScopeLock Lock(&ActionDurationCS);

	if (bActionDurationElapsed)
	{
		bActionDurationElapsed = false;
		return true;
	}

	return false;
}

#if WITH_GAMEPLAY_DEBUGGER
#include "GameplayDebuggerCategory.h"

void UMLAdapterAgent::DescribeSelfToGameplayDebugger(FGameplayDebuggerCategory& DebuggerCategory) const
{
	DebuggerCategory.AddTextLine(FString::Printf(TEXT("{green}ID {white}%u\n{green}Avatar {white}%s")
		, AgentID, *GetNameSafe(Avatar)));

	DebuggerCategory.AddTextLine(TEXT("{green}Sensors:"));
	for (UMLAdapterSensor* Sensor : Sensors)
	{
		if (Sensor)
		{
			Sensor->DescribeSelfToGameplayDebugger(DebuggerCategory);
		}
	}

	DebuggerCategory.AddTextLine(TEXT("{green}Actuators:"));
	for (UMLAdapterActuator* Actuator : Actuators)
	{
		if (Actuator)
		{
			Actuator->DescribeSelfToGameplayDebugger(DebuggerCategory);
		}
	}
}
#endif // WITH_GAMEPLAY_DEBUGGER