// Copyright Epic Games, Inc. All Rights Reserved.

#include "Agents/MLAdapterAgentElement.h"
#include "GameFramework/Controller.h"
#include "GameFramework/Pawn.h"
#include "Agents/MLAdapterAgent.h"


UMLAdapterAgentElement::UMLAdapterAgentElement(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, SpaceDef(MakeShareable(new FMLAdapter::FSpace_Dummy()))
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		Description = FString::Printf(TEXT("%s, detailed description pending"), *GetClass()->GetName());
	}

	Nickname = GetName();
}

void UMLAdapterAgentElement::PostInitProperties()
{
	// UMLAdapterAgent instance is the only valid outer type
	check(HasAnyFlags(RF_ClassDefaultObject) || Cast<UMLAdapterAgent>(GetOuter()));

	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		UpdateSpaceDef();
	}
}

const UMLAdapterAgent& UMLAdapterAgentElement::GetAgent() const
{
	// UMLAdapterAgent instance is the only valid outer type
	return *CastChecked<UMLAdapterAgent>(GetOuter());
}

AActor* UMLAdapterAgentElement::GetAvatar() const
{
	return GetAgent().GetAvatar();
}

APawn* UMLAdapterAgentElement::GetPawnAvatar() const
{
	AActor* Avatar = GetAvatar();
	APawn* Pawn = Cast<APawn>(Avatar);
	AController* Controller = Cast<AController>(Avatar);
	return Pawn ? Pawn : (Controller ? Controller->GetPawn().Get() : nullptr);
}

AController* UMLAdapterAgentElement::GetControllerAvatar() const
{
	AActor* Avatar = GetAvatar();
	APawn* Pawn = Cast<APawn>(Avatar);
	AController* Controller = Cast<AController>(Avatar);
	return Controller ? Controller : (Pawn ? Pawn->GetController() : nullptr);
}

bool UMLAdapterAgentElement::GetPawnAndControllerAvatar(APawn*& OutPawn, AController*& OutController) const
{
	AActor* Avatar = GetAvatar();
	APawn* Pawn = Cast<APawn>(Avatar);
	AController* Controller = Cast<AController>(Avatar);
	if (Pawn)
	{
		Controller = Pawn->GetController();
	}
	else if (Controller)
	{
		Pawn = Controller->GetPawn();
	}
	OutPawn = Pawn;
	OutController = Controller;
	
	return Pawn || Controller;
}

void UMLAdapterAgentElement::Configure(const TMap<FName, FString>& Params)
{
	const FName NAME_Nickname = TEXT("nickname");

	for (auto KeyValue : Params)
	{
		if (KeyValue.Key == NAME_Nickname)
		{
			Nickname = KeyValue.Value;
		}
	}
}

void UMLAdapterAgentElement::UpdateSpaceDef()
{
	SpaceDef = ConstructSpaceDef().ToSharedRef();
}

#if WITH_GAMEPLAY_DEBUGGER
#include "GameplayDebuggerCategory.h"

void UMLAdapterAgentElement::DescribeSelfToGameplayDebugger(FGameplayDebuggerCategory& DebuggerCategory) const
{
	DebuggerCategory.AddTextLine(FString::Printf(TEXT("\t{yellow}[%d] %s {white}%s"), ElementID
		, *GetName(), *DebugRuntimeString));
}
#endif // WITH_GAMEPLAY_DEBUGGER
