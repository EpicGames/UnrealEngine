// Copyright Epic Games, Inc. All Rights Reserved.

#include "Agents/MLAdapterAgent_Inference.h"
#include "HAL/UnrealMemory.h"
#include "MLAdapterTypes.h"
#include "NNE.h"
#include "NNETypes.h"

void UMLAdapterAgent_Inference::PostInitProperties()
{
	const FString RuntimeName = TEXT("NNERuntimeORTCpu");

	Super::PostInitProperties();
	
	if (ModelData == nullptr)
	{
		UE_LOGF(LogMLAdapter, Warning, "Neural network asset data not set");
		return;
	}

	TWeakInterfacePtr<INNERuntimeCPU> Runtime = UE::NNE::GetRuntime<INNERuntimeCPU>(RuntimeName);
	if (!Runtime.IsValid())
	{
		UE_LOGF(LogMLAdapter, Warning, "Runtime %ls is not valid!", *RuntimeName);
		return;
	}

	Brain = Runtime->CreateModelCPU(ModelData)->CreateModelInstanceCPU();

	TConstArrayView<UE::NNE::FTensorDesc> InputTensorDescs = Brain->GetInputTensorDescs();
	if (InputTensorDescs.Num() != 1)
	{
		UE_LOGF(LogMLAdapter, Warning, "Brain accepts only single tensor input.");
		return;
	}
	if (!InputTensorDescs[0].GetShape().IsConcrete())
	{
		UE_LOGF(LogMLAdapter, Warning, "Brain does not accept dynamic tensor input shapes.");
		return;
	}

	UE::NNE::FTensorShape InputTensorShape = UE::NNE::FTensorShape::MakeFromSymbolic(InputTensorDescs[0].GetShape());

	Brain->SetInputTensorShapes({ InputTensorShape });

	TConstArrayView<UE::NNE::FTensorDesc> OutputTensorDescs = Brain->GetOutputTensorDescs();
	if (OutputTensorDescs.Num() != 1)
	{
		UE_LOGF(LogMLAdapter, Warning, "Brain accepts only single tensor output.");
		return;
	}

	TConstArrayView<UE::NNE::FTensorShape> OutputTensorShapes = Brain->GetOutputTensorShapes();
	if (OutputTensorShapes.Num() != 1)
	{
		UE_LOGF(LogMLAdapter, Warning, "Brain could not resolve output shapes.");
		return;
	}

	InputTensorSizeInBytes = InputTensorDescs[0].GetElementByteSize() * InputTensorShape.Volume();
	if (InputTensorSizeInBytes == 0)
	{
		UE_LOGF(LogMLAdapter, Warning, "Input tensor should be of non-zero size.");
		return;
	}

	OutputTensorSizeInBytes = OutputTensorDescs[0].GetElementByteSize() * OutputTensorShapes[0].Volume();
	if (OutputTensorSizeInBytes == 0)
	{
		UE_LOGF(LogMLAdapter, Warning, "Output tensor should be of non-zero size.");
		return;
	}
}

void UMLAdapterAgent_Inference::Think(const float DeltaTime)
{
	if (Brain == nullptr)
	{
		UE_LOGF(LogMLAdapter, Warning, "Agent beginning to Think but Brain is null, Skipping.");
		return;
	}

	if (bEnableActionDuration && !bActionDurationElapsed)
	{
		// Action duration is enabled and it's not time to think yet.
		return;
	}

	if (InputTensorSizeInBytes == 0 || OutputTensorSizeInBytes == 0)
	{
		return;
	}

	bActionDurationElapsed = false;

	TArray<uint8> Buffer;
	FMLAdapterMemoryWriter Writer(Buffer);
	GetObservations(Writer);

	if ((uint32)Buffer.Num() < InputTensorSizeInBytes)
	{
		UE_LOGF(LogMLAdapter, Warning, "Brain received not enough data.");
		return;
	}
	
	TArray<uint8> OutputBuffer;
	OutputBuffer.SetNumUninitialized(OutputTensorSizeInBytes);

	UE::NNE::FTensorBindingCPU InputTensorBinding{Buffer.GetData(), Buffer.Num()};
	UE::NNE::FTensorBindingCPU OutputTensorBinding{OutputBuffer.GetData(), OutputBuffer.Num()};

	Brain->RunSync({InputTensorBinding}, {OutputTensorBinding});

	FMLAdapterMemoryReader Reader(OutputBuffer);
	DigestActions(Reader);
}
