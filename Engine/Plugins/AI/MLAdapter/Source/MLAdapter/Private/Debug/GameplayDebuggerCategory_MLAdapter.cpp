// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/GameplayDebuggerCategory_MLAdapter.h"
#include "Engine/Engine.h"
#include "Misc/CoreDelegates.h"
#include "Managers/MLAdapterManager.h"
#include "Sessions/MLAdapterSession.h"
#include "Agents/MLAdapterAgent.h"
#include "Sensors/MLAdapterSensor.h"
#include "Actuators/MLAdapterActuator.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU
#include "GameplayDebuggerCategoryReplicator.h"

//----------------------------------------------------------------------//
//  FGameplayDebuggerCategory_MLAdapter
//----------------------------------------------------------------------//
FGameplayDebuggerCategory_MLAdapter::FGameplayDebuggerCategory_MLAdapter()
{
	CachedAgentID = FMLAdapter::InvalidAgentID;
	CachedDebugActor = nullptr;
	bShowOnlyWithDebugActor = false;
	
	BindKeyPress(EKeys::RightBracket.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_MLAdapter::OnShowNextAgent, EGameplayDebuggerInputMode::Replicated);
	BindKeyPress(EKeys::LeftBracket.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_MLAdapter::OnRequestAvatarUpdate, EGameplayDebuggerInputMode::Replicated);
	BindKeyPress(EKeys::P.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_MLAdapter::OnSetAvatarAsDebugAgent, EGameplayDebuggerInputMode::Replicated);
}

void FGameplayDebuggerCategory_MLAdapter::Init()
{
	UMLAdapterManager::Get().GetOnCurrentSessionChanged().AddSP(this, &FGameplayDebuggerCategory_MLAdapter::OnCurrentSessionChanged);
	if (UMLAdapterManager::Get().HasSession())
	{
		OnCurrentSessionChanged();
	}
}

void FGameplayDebuggerCategory_MLAdapter::ResetProps()
{
	CachedAgentID = FMLAdapter::InvalidAgentID;
	CachedDebugActor = nullptr;
}

TSharedRef<FGameplayDebuggerCategory> FGameplayDebuggerCategory_MLAdapter::MakeInstance()
{
	FGameplayDebuggerCategory_MLAdapter* Instance = new FGameplayDebuggerCategory_MLAdapter();
	TSharedRef<FGameplayDebuggerCategory> SharedInstanceRef = MakeShareable(Instance);

	if (UMLAdapterManager::IsReady())
	{
		Instance->Init();
	}
	else
	{
		UMLAdapterManager::OnPostInit.AddSP(Instance, &FGameplayDebuggerCategory_MLAdapter::Init);
	}

	return SharedInstanceRef;
}

void FGameplayDebuggerCategory_MLAdapter::CollectData(APlayerController* OwnerPC, AActor* DebugActor)
{
	if (UMLAdapterManager::Get().HasSession() == false)
	{
		AddTextLine(TEXT("{red}No session"));
		return;
	}

	UMLAdapterSession& Session = UMLAdapterManager::Get().GetSession();

	const UMLAdapterAgent* Agent = nullptr;
	if (CachedDebugActor != DebugActor)
	{
		CachedDebugActor = DebugActor;
		if (DebugActor)
		{
			Agent = Session.FindAgentByAvatar(*DebugActor);
			
		}
		CachedAgentID = Agent ? Agent->GetAgentID() : FMLAdapter::InvalidAgentID;
	}

	if (CachedAgentID != FMLAdapter::InvalidAgentID && Agent == nullptr)
	{
		Agent = Session.GetAgent(CachedAgentID);
		ensureMsgf(Agent, TEXT("Null-agent retrieved while AgentID used was valid"));
	}

	Session.DescribeSelfToGameplayDebugger(*this);
	AddTextLine(FString::Printf(TEXT("{DimGrey}---------------------")));
	if (Agent)
	{
		Agent->DescribeSelfToGameplayDebugger(*this);
	}
	else if (CachedAgentID != FMLAdapter::InvalidAgentID)
	{
		AddTextLine(FString::Printf(TEXT("{orange}Agent %d has no avatar"), CachedAgentID));
	}
	else
	{
		AddTextLine(TEXT("{orange}No agent selected"));
	}
}

void FGameplayDebuggerCategory_MLAdapter::DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext)
{
	CanvasContext.Printf(TEXT("\n[{yellow}%s{white}] Next agent"), *GetInputHandlerDescription(0));
	CanvasContext.Printf(TEXT("[{yellow}%s{white}] Request avatar"), *GetInputHandlerDescription(1));
	CanvasContext.Printf(TEXT("[{yellow}%s{white}] Debug current avatar"), *GetInputHandlerDescription(2));

	FGameplayDebuggerCategory::DrawData(OwnerPC, CanvasContext);
}

void FGameplayDebuggerCategory_MLAdapter::OnShowNextAgent()
{
	CachedDebugActor = nullptr;
	// should get called on Authority
	if (UMLAdapterManager::Get().HasSession())
	{
		CachedAgentID = UMLAdapterManager::Get().GetSession().GetNextAgentID(CachedAgentID);
		if (CachedAgentID != FMLAdapter::InvalidAgentID)
		{
			const UMLAdapterAgent* Agent = UMLAdapterManager::Get().GetSession().GetAgent(CachedAgentID);
			if (ensure(Agent) && Agent->GetAvatar())
			{
				AGameplayDebuggerCategoryReplicator* Replicator = GetReplicator();
				if (ensure(Replicator))
				{
					Replicator->SetDebugActor(Agent->GetAvatar());
				}
			}
		}
	}
}

void FGameplayDebuggerCategory_MLAdapter::OnRequestAvatarUpdate()
{
	if (UMLAdapterManager::Get().HasSession() && CachedAgentID != FMLAdapter::InvalidAgentID)
	{
		UMLAdapterManager::Get().GetSession().RequestAvatarForAgent(CachedAgentID);
	}
}

void FGameplayDebuggerCategory_MLAdapter::OnSetAvatarAsDebugAgent()
{
	if (CachedAgentID != FMLAdapter::InvalidAgentID && UMLAdapterManager::Get().HasSession())
	{
		const UMLAdapterAgent* Agent = UMLAdapterManager::Get().GetSession().GetAgent(CachedAgentID);
		if (ensure(Agent) && Agent->GetAvatar())
		{
			AGameplayDebuggerCategoryReplicator* Replicator = GetReplicator();
			if (ensure(Replicator))
			{
				Replicator->SetDebugActor(Agent->GetAvatar());
			}
		}
	}
}

void FGameplayDebuggerCategory_MLAdapter::OnCurrentSessionChanged()
{
	if (UMLAdapterManager::Get().HasSession())
	{
		UMLAdapterManager::Get().GetSession().GetOnAgentAvatarChanged().AddSP(this, &FGameplayDebuggerCategory_MLAdapter::OnAgentAvatarChanged);
		UMLAdapterManager::Get().GetSession().GetOnBeginAgentRemove().AddSP(this, &FGameplayDebuggerCategory_MLAdapter::OnBeginAgentRemove);
	}
	else
	{
		ResetProps();
	}
}

void FGameplayDebuggerCategory_MLAdapter::OnAgentAvatarChanged(UMLAdapterAgent& Agent, AActor* OldAvatar)
{
	if (Agent.GetAgentID() == CachedAgentID)
	{
		// pass
	}
}

void FGameplayDebuggerCategory_MLAdapter::OnBeginAgentRemove(UMLAdapterAgent& Agent)
{
	if (Agent.GetAgentID() == CachedAgentID)
	{
		ResetProps();
	}
}

#endif // WITH_GAMEPLAY_DEBUGGER_MENU

