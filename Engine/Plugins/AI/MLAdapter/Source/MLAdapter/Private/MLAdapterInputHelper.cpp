// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLAdapterInputHelper.h"
#include "GameFramework/InputSettings.h"


namespace FMLAdapterInputHelper
{
	void CreateInputMap(TArray<TTuple<FKey, FName>>& InterfaceKeys, TMap<FKey, int32>& FKeyToInterfaceKeyMap)
	{
		UInputSettings* InputSettings = UInputSettings::GetInputSettings();
		if (ensure(InputSettings))
		{
			TMap<FName, TArray<FKey>> AvailableActions;

			for (const FInputActionKeyMapping& ActionKey : InputSettings->GetActionMappings())
			{
				TArray<FKey>& Keys = AvailableActions.FindOrAdd(ActionKey.ActionName);
				Keys.Add(ActionKey.Key);
			}

			for (const FInputAxisKeyMapping& AxisAction : InputSettings->GetAxisMappings())
			{
				const bool bIsKeyboard = AxisAction.Key.IsGamepadKey() == false && AxisAction.Key.IsMouseButton() == false;
				const FString ActionName = FString::Printf(TEXT("%s%s")
					, AxisAction.Scale > 0 ? TEXT("+") : TEXT("-")
					, *AxisAction.AxisName.ToString());
				const FName ActionFName(*ActionName);

				if (bIsKeyboard && AvailableActions.Find(ActionFName) == nullptr)
				{
					AvailableActions.Add(ActionFName, { AxisAction.Key });
				}
			}

			for (auto KeyValue : AvailableActions)
			{
				const int32 Index = InterfaceKeys.Add(TTuple<FKey, FName>(KeyValue.Value[0], KeyValue.Key));
				for (auto Key : KeyValue.Value)
				{
					ensure(FKeyToInterfaceKeyMap.Find(Key) == nullptr);
					FKeyToInterfaceKeyMap.Add(Key, Index);
				}
			}
		}
	}

	TSharedPtr<FMLAdapter::FSpace> ConstructEnhancedInputSpaceDef(const TArray<UInputAction*>& TrackedActions)
	{
		uint32 DiscreteActionCount = 0;
		uint32 AxisActionCount = 0;

		for (const UInputAction* InputAction : TrackedActions)
		{
			switch (InputAction->ValueType)
			{
			case EInputActionValueType::Boolean:
				++DiscreteActionCount;
				break;
			case EInputActionValueType::Axis1D:
				AxisActionCount += 1;
				break;
			case EInputActionValueType::Axis2D:
				AxisActionCount += 2;
				break;
			case EInputActionValueType::Axis3D:
				AxisActionCount += 3;
				break;
			default:
				checkf(false, TEXT("Unsupported value type for input action value!"));
				break;
			}
		}

		FMLAdapter::FSpace* Result = nullptr;

		if (DiscreteActionCount > 0 && AxisActionCount > 0)
		{
			Result = new FMLAdapter::FSpace_Tuple({
				MakeShareable(new FMLAdapter::FSpace_MultiDiscrete(DiscreteActionCount)),
				MakeShareable(new FMLAdapter::FSpace_Box({AxisActionCount})) });
		}
		else if (DiscreteActionCount > 0)
		{
			Result = new FMLAdapter::FSpace_MultiDiscrete(DiscreteActionCount);
		}
		else if (AxisActionCount > 0)
		{
			Result = new FMLAdapter::FSpace_Box({ AxisActionCount });
		}
		else
		{
			Result = new FMLAdapter::FSpace_Dummy();
		}

		return MakeShareable(Result);
	}
}