// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLAdapterLibrarian.h"
#include "MLAdapterTypes.h"
#include "Sensors/MLAdapterSensor.h"
#include "Actuators/MLAdapterActuator.h"
#include "Agents/MLAdapterAgent.h"
#include "UObject/UObjectHash.h"
#include "Managers/MLAdapterManager.h"


namespace FLibrarianHelper
{
	bool IsValidClass(const UClass* Class)
	{
		if (!Class)
		{
			return false;
		}

		if (Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			UE_LOGF(LogMLAdapter, Log, "Librarian: skipping class %ls registration due to it being %ls%ls%ls"
				, *Class->GetName()
				, Class->HasAnyClassFlags(CLASS_Abstract) ? TEXT("Abstract, ") : TEXT("")
				, Class->HasAnyClassFlags(CLASS_Deprecated) ? TEXT("Deprecated, ") : TEXT("")
				, Class->HasAnyClassFlags(CLASS_NewerVersionExists) ? TEXT("NewerVersionExists, ") : TEXT("")
			);
			return false;
		}

		return true;
	}
}

const FMLAdapterLibrarian& FMLAdapterLibrarian::Get()
{
	return UMLAdapterManager::Get().GetLibrarian();
}

void FMLAdapterLibrarian::GatherClasses()
{
	{
		TArray<UClass*> Results;
		GetDerivedClasses(UMLAdapterSensor::StaticClass(), Results, /*bRecursive=*/true);
		for (UClass* Class : Results)
		{
			RegisterSensorClass(Class);
		}
	}
	{
		TArray<UClass*> Results;
		GetDerivedClasses(UMLAdapterActuator::StaticClass(), Results, /*bRecursive=*/true);
		for (UClass* Class : Results)
		{
			RegisterActuatorClass(Class);
		}
	}
	{
		RegisterAgentClass(UMLAdapterAgent::StaticClass());

		TArray<UClass*> Results;
		GetDerivedClasses(UMLAdapterAgent::StaticClass(), Results, /*bRecursive=*/true);
		for (UClass* Class : Results)
		{
			RegisterAgentClass(Class);
		}
	}
}

void FMLAdapterLibrarian::RegisterSensorClass(const TSubclassOf<UMLAdapterSensor>& Class)
{
	if (FLibrarianHelper::IsValidClass(Class) == false)
	{
		return;
	}

	UMLAdapterSensor* CDO = Class->GetDefaultObject<UMLAdapterSensor>();
	check(CDO);
	if (KnownSensorClasses.Find(CDO->GetElementID()) == nullptr)
	{
		KnownSensorClasses.Add(CDO->GetElementID(), Class);
	}
}

void FMLAdapterLibrarian::RegisterActuatorClass(const TSubclassOf<UMLAdapterActuator>& Class)
{
	if (FLibrarianHelper::IsValidClass(Class) == false)
	{
		return;
	}

	UMLAdapterActuator* CDO = Class->GetDefaultObject<UMLAdapterActuator>();
	check(CDO);
	if (KnownActuatorClasses.Find(CDO->GetElementID()) == nullptr)
	{
		KnownActuatorClasses.Add(CDO->GetElementID(), Class);
	}
}

void FMLAdapterLibrarian::RegisterAgentClass(const TSubclassOf<UMLAdapterAgent>& Class)
{
	KnownAgentClasses.AddUnique(Class);
}

void FMLAdapterLibrarian::AddRPCFunctionDescription(const FName FunctionName, FString&& Description)
{
	RPCFunctionDescriptions.FindOrAdd(FunctionName) = Description;
}

bool FMLAdapterLibrarian::GetFunctionDescription(const FName FunctionName, FString& OutDescription) const
{
	const FString* FoundDesc = RPCFunctionDescriptions.Find(FunctionName);
	if (FoundDesc)
	{
		OutDescription = *FoundDesc;
	}
	return FoundDesc != nullptr;
}

bool FMLAdapterLibrarian::GetSensorDescription(const FName SensorName, FString& OutDescription) const
{
	UClass* ResultClass = FindSensorClass(SensorName);
	if (ResultClass)
	{
		UMLAdapterAgentElement* CDO = ResultClass->GetDefaultObject<UMLAdapterAgentElement>();
		if (CDO)
		{
			OutDescription = CDO->GetDescription();
			return true;
		}
	}
	return false;
}

bool FMLAdapterLibrarian::GetActuatorDescription(const FName ActuatorName, FString& OutDescription) const
{
	UClass* ResultClass = FindActuatorClass(ActuatorName);
	if (ResultClass)
	{
		UMLAdapterAgentElement* CDO = ResultClass->GetDefaultObject<UMLAdapterAgentElement>();
		if (CDO)
		{
			OutDescription = CDO->GetDescription();
			return true;
		}
	}
	return false;
}

TSubclassOf<UMLAdapterAgent> FMLAdapterLibrarian::FindAgentClass(const FName ClassName) const
{
	UClass* AgentClass = nullptr;
	if (ClassName != NAME_None)
	{
		for (auto It = GetAgentsClassIterator(); It; ++It)
		{
			if (It->Get() && It->Get()->GetFName() == ClassName)
			{
				AgentClass = *It;
				break;
			}
		}

		if (AgentClass == nullptr)
		{
			const FName DecoratedName(FString::Printf(TEXT("MLAdapterAgent_%s"), *ClassName.ToString()));
			for (auto It = GetAgentsClassIterator(); It; ++It)
			{
				if (It->Get() && It->Get()->GetFName() == DecoratedName)
				{
					AgentClass = *It;
					break;
				}
			}
		}
	}

	return (AgentClass != nullptr) ? AgentClass : UMLAdapterAgent::StaticClass();
}

TSubclassOf<UMLAdapterSensor> FMLAdapterLibrarian::FindSensorClass(const FName ClassName) const
{
	UClass* ResultClass = nullptr;
	if (ClassName != NAME_None)
	{
		for (auto It = GetSensorsClassIterator(); It; ++It)
		{
			if (It->Value.Get() && It->Value.Get()->GetFName() == ClassName)
			{
				ResultClass = It->Value.Get();
				break;
			}
		}

		if (ResultClass == nullptr)
		{
			const FName DecoratedName(FString::Printf(TEXT("MLAdapterSensor_%s"), *ClassName.ToString()));
			for (auto It = GetSensorsClassIterator(); It; ++It)
			{
				if (It->Value.Get() && It->Value.Get()->GetFName() == DecoratedName)
				{
					ResultClass = It->Value.Get();
					break;
				}
			}
		}
	}

	return ResultClass;
}

TSubclassOf<UMLAdapterActuator> FMLAdapterLibrarian::FindActuatorClass(const FName ClassName) const
{
	UClass* ResultClass = nullptr;
	if (ClassName != NAME_None)
	{
		for (auto It = GetActuatorsClassIterator(); It; ++It)
		{
			if (It->Value.Get() && It->Value.Get()->GetFName() == ClassName)
			{
				ResultClass = It->Value.Get();
				break;
			}
		}

		if (ResultClass == nullptr)
		{
			const FName DecoratedName(FString::Printf(TEXT("MLAdapterActuator_%s"), *ClassName.ToString()));
			for (auto It = GetActuatorsClassIterator(); It; ++It)
			{
				if (It->Value.Get() && It->Value.Get()->GetFName() == DecoratedName)
				{
					ResultClass = It->Value.Get();
					break;
				}
			}
		}
	}

	return ResultClass;
}
