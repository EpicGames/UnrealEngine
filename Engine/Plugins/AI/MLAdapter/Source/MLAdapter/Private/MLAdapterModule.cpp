// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLAdapterModule.h"
#include "Managers/MLAdapterManager.h"
#if WITH_GAMEPLAY_DEBUGGER
#include "GameplayDebugger.h"
#include "Debug/GameplayDebuggerCategory_MLAdapter.h"
#endif // WITH_GAMEPLAY_DEBUGGER


class FMLAdapterModule : public IMLAdapterModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE( FMLAdapterModule, MLAdapter )

void FMLAdapterModule::StartupModule()
{
#if WITH_GAMEPLAY_DEBUGGER
	IGameplayDebugger& GameplayDebuggerModule = IGameplayDebugger::Get();
	GameplayDebuggerModule.RegisterCategory("MLAdapter", IGameplayDebugger::FOnGetCategory::CreateStatic(&FGameplayDebuggerCategory_MLAdapter::MakeInstance), EGameplayDebuggerCategoryState::EnabledInGameAndSimulate);
	GameplayDebuggerModule.NotifyCategoriesChanged();
#endif
}

void FMLAdapterModule::ShutdownModule()
{
#if WITH_GAMEPLAY_DEBUGGER
	if (IGameplayDebugger::IsAvailable())
	{
		IGameplayDebugger& GameplayDebuggerModule = IGameplayDebugger::Get();
		GameplayDebuggerModule.UnregisterCategory("MLAdapter");
		GameplayDebuggerModule.NotifyCategoriesChanged();
	}
#endif
}
