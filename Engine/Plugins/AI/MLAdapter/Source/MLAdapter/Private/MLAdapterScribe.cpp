// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLAdapterScribe.h"
#include "Managers/MLAdapterManager.h"
#include "MLAdapterLibrarian.h"

#define FSTRING_TO_STD(a) (std::string(TCHAR_TO_UTF8(*a)))

namespace FMLAdapterScribe
{
	std::vector<std::string> ToStringVector(const TArray<FString>& Array)
	{
		std::vector<std::string> Ret;
		Ret.reserve(Array.Num());
		for (auto String : Array)
		{
			Ret.push_back(FSTRING_TO_STD(String));
		}
		return Ret;
	}

	std::vector<std::string> ToStringVector(const TArray<FName>& Array)
	{
		std::vector<std::string> Ret;
		Ret.reserve(Array.Num());
		for (auto Name : Array)
		{
			Ret.push_back(FSTRING_TO_STD(Name.ToString()));
		}
		return Ret;
	}

	std::vector<std::string> ListFunctions()
	{
		std::vector<std::string> Ret;
		const FMLAdapterLibrarian& Librarian = FMLAdapterLibrarian::Get();
		for (TMap<FName, FString>::TConstIterator It = Librarian.GetFunctionDescriptionsIterator(); It; ++It)
		{
			Ret.push_back(FSTRING_TO_STD(It->Key.ToString().ToLower()));
		}
		return Ret;
	}

	std::map<std::string, uint32> ListSensorTypes()
	{
		std::map<std::string, uint32> Ret;

		const FMLAdapterLibrarian& Librarian = UMLAdapterManager::Get().GetLibrarian();
		for (auto It = Librarian.GetSensorsClassIterator(); It; ++It)
		{
			Ret[FSTRING_TO_STD(It->Value->GetName())] = It->Key;
		}

		return Ret;
	}

	std::map<std::string, uint32> ListActuatorTypes() 
	{
		std::map<std::string, uint32> Ret;

		const FMLAdapterLibrarian& Librarian = UMLAdapterManager::Get().GetLibrarian();
		for (auto It = Librarian.GetActuatorsClassIterator(); It; ++It)
		{
			Ret[FSTRING_TO_STD(It->Value->GetName())] = It->Key;
		}

		return Ret;
	}

	std::string GetDescription(std::string const& ElementName)
	{
		const FMLAdapterLibrarian& Librarian = UMLAdapterManager::Get().GetLibrarian();
		const FName AsFName = FName(FString(ElementName.c_str()));
		FString Description;

		if (Librarian.GetFunctionDescription(AsFName, Description)
			|| Librarian.GetSensorDescription(AsFName, Description)
			|| Librarian.GetActuatorDescription(AsFName, Description))
		{
			return FSTRING_TO_STD(Description);
		}

		return std::string("Not Found");
	}
}