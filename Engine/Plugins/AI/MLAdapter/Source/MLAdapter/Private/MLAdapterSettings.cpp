// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLAdapterSettings.h"

#define GET_CONFIG_VALUE(a) (GetDefault<UMLAdapterSettings>()->a)

UMLAdapterSettings::UMLAdapterSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DefaultAgentClass = UMLAdapterAgent::StaticClass();
	ManagerClass = UMLAdapterManager::StaticClass();
	SessionClass = UMLAdapterSession::StaticClass();
}

TSubclassOf<UMLAdapterManager> UMLAdapterSettings::GetManagerClass()
{
	const FSoftClassPath LocalClassName = GET_CONFIG_VALUE(ManagerClass);
	TSubclassOf<UMLAdapterManager> LocalClass = LocalClassName.TryLoadClass<UMLAdapterManager>();
	return LocalClass;
}

TSubclassOf<UMLAdapterSession> UMLAdapterSettings::GetSessionClass()
{
	const FSoftClassPath LocalClassName = GET_CONFIG_VALUE(SessionClass);
	TSubclassOf<UMLAdapterSession> LocalClass = LocalClassName.TryLoadClass<UMLAdapterSession>();
	return LocalClass;
}

TSubclassOf<UMLAdapterAgent> UMLAdapterSettings::GetAgentClass()
{
	const FSoftClassPath LocalClassName = GET_CONFIG_VALUE(DefaultAgentClass);
	TSubclassOf<UMLAdapterAgent> LocalClass = LocalClassName.TryLoadClass<UMLAdapterAgent>();
	return LocalClass;
}

#if WITH_EDITOR
void UMLAdapterSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UMLAdapterSettings, ManagerClass))
	{
		UMLAdapterManager::RecreateManagerInstance();
	}
}
#endif // WITH_EDITOR

#undef GET_CONFIG_VALUE