// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLAdapterSpace.h"
#include "MLAdapterTypes.h"
#include "MLAdapterJson.h"


DEFINE_ENUM_TO_STRING(EMLAdapterSpaceType, "/Script/MLAdapter");

namespace FMLAdapter
{
	//----------------------------------------------------------------------//
	// FSpace_Discrete
	//----------------------------------------------------------------------//
	FSpace_Discrete::FSpace_Discrete(uint32 InCount) : Count(InCount)
	{
		Type = EMLAdapterSpaceType::Discrete;
	}

	FString FSpace_Discrete::ToJson() const
	{
		return FString::Printf(TEXT("{\"%s\":%d}"), *EnumToString(Type), Count);
	}

	//----------------------------------------------------------------------//
	// FSpace_MultiDiscrete
	//----------------------------------------------------------------------//
	FSpace_MultiDiscrete::FSpace_MultiDiscrete(uint32 InCount, uint32 InValues)
	{
		Type = EMLAdapterSpaceType::MultiDiscrete;
		Options.AddUninitialized(InCount);
		for (uint32 Index = 0; Index < InCount; ++Index)
		{
			Options[Index] = InValues;
		}
	}

	FSpace_MultiDiscrete::FSpace_MultiDiscrete(std::initializer_list<uint32> InOptions)
		: Options(InOptions)
	{
		Type = EMLAdapterSpaceType::MultiDiscrete;
	}

	FSpace_MultiDiscrete::FSpace_MultiDiscrete(const TArray<uint32>& InOptions)
		: Options(InOptions)
	{
		Type = EMLAdapterSpaceType::MultiDiscrete;
	}

	FString FSpace_MultiDiscrete::ToJson() const
	{
		FString ShapeString;
		for (uint32 Option : Options)
		{
			ShapeString += FString::Printf(TEXT("%d,"), Option);
		}
		// python-side json parsing doesn't like dangling commas
		ShapeString.RemoveAt(ShapeString.Len() - 1, EAllowShrinking::No);

		return FString::Printf(TEXT("{\"%s\":[%s]}"), *EnumToString(Type), *ShapeString);
	}

	//----------------------------------------------------------------------//
	// FSpace_Box
	//----------------------------------------------------------------------//
	FSpace_Box::FSpace_Box()
	{
		Type = EMLAdapterSpaceType::Box;
	}

	FSpace_Box::FSpace_Box(std::initializer_list<uint32> InShape, float InLow, float InHigh)
		: Low(InLow), High(InHigh)
	{
		Shape = InShape;
		ensure(Shape.Num() > 0);
		Type = EMLAdapterSpaceType::Box;
	}

	FString FSpace_Box::ToJson() const
	{
		if (!ensure(Shape.Num() > 0))
		{
			return TEXT("{\"error\":\"No shape\"}");
		}
		if (Shape.Num() == 1)
		{
			return FString::Printf(TEXT("{\"%s\":[%f,%f,%d]}"), *EnumToString(Type), Low, High, Shape[0]);
		}

		FString ShapeString;
		for (uint32 Size : Shape)
		{
			ShapeString += FString::Printf(TEXT("%d,"), Size);
		}
		// python-side json parsing doesn't like dangling commas
		ShapeString.RemoveAt(ShapeString.Len() - 1, EAllowShrinking::No);
		return FString::Printf(TEXT("{\"%s\":[%f,%f,%s]}"), *EnumToString(Type), Low, High, *ShapeString);
	}

	int32 FSpace_Box::Num() const
	{
		uint32 Result = 1;
		for (uint32 N : Shape)
		{
			Result *= N;
		}
		return int32(Result);
	}

	//----------------------------------------------------------------------//
	// FSpace_Tuple
	//----------------------------------------------------------------------//
	FSpace_Tuple::FSpace_Tuple() 
	{
		Type = EMLAdapterSpaceType::Tuple;
	}

	FSpace_Tuple::FSpace_Tuple(std::initializer_list<TSharedPtr<FSpace> > InitList)
		: SubSpaces(InitList)
	{
		Type = EMLAdapterSpaceType::Tuple;
	}

	FSpace_Tuple::FSpace_Tuple(TArray<TSharedPtr<FSpace> >& InSubSpaces)
		: SubSpaces(InSubSpaces)
	{
		Type = EMLAdapterSpaceType::Tuple;
	}

	FString FSpace_Tuple::ToJson() const
	{
		FString Contents;
		for (const TSharedPtr<FSpace>& Space : SubSpaces)
		{
			Contents += Space->ToJson();
			Contents += TEXT(",");
		}
		Contents.RemoveAt(Contents.Len() - 1, EAllowShrinking::No);
		return FString::Printf(TEXT("{\"%s\":[%s]}"), *EnumToString(Type), *Contents);
	}

	int32 FSpace_Tuple::Num() const
	{
		int32 Result = 0;
		for (const TSharedPtr<FSpace>& Space : SubSpaces)
		{
			Result += Space->Num();
		}
		return Result;
	}
}

//----------------------------------------------------------------------//
// FMLAdapterSpaceDescription
//----------------------------------------------------------------------//
bool FMLAdapterDescription::FromJson(const FString& JsonString, FMLAdapterDescription& OutInstance)
{
	//FMLAdapter::JsonStringToStruct(JsonString, OutInstance);
	ensureMsgf(false, TEXT("FMLAdapterDescription::FromJson not implemented"));

	return (OutInstance.IsEmpty() == false);
}

bool PairArrayToJson(const TArray<TPair<FString, FString>>& Array, TSharedRef<FJsonObject>& Out)
{
	for (const auto& Pair : Array)
	{
		Out->SetField(Pair.Get<0>(), MakeShared<FJsonValueString>(Pair.Get<1>()));
	}
	return true;
}

bool PairArrayToJson(const TArray<TPair<FString, FMLAdapterDescription>>& Array, TSharedRef<FJsonObject>& Out)
{
	for (const auto& Pair : Array)
	{
		Out->SetField(Pair.Get<0>(), MakeShared<FJsonValueString>(Pair.Get<1>().ToJson()));
	}
	return true;
}

bool MapToJson(const TMap<FString, FString>& Map, TSharedRef<FJsonObject>& Out)
{
	for (const auto& KeyVal : Map)
	{
		Out->SetField(KeyVal.Key, MakeShared<FJsonValueString>(KeyVal.Value));
	}

	return true;
}
	
FString FMLAdapterDescription::ToJson() const
{
	FString RetString;

	if (PrepData.Num() > 0)
	{
		if (PrepData.Num() > 1)
		{
			RetString += TEXT("[");
		}
		for (auto Str : PrepData)
		{
			RetString += Str;
			RetString += TEXT(",");
		}
		// pop the last ,
		RetString.RemoveAt(RetString.Len() - 1, EAllowShrinking::No);
		if (PrepData.Num() > 1)
		{
			RetString += TEXT("]");
		}
	}
	else
	{
		TSharedRef<FJsonObject> JsonObject = MakeShareable(new FJsonObject());

		bool bValid = false;
		//bValid = MapToJson(Data, JsonObject);
		bValid = PairArrayToJson(Data, JsonObject);

		TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&RetString, /*Indent=*/0);

		if (FJsonSerializer::Serialize(JsonObject, JsonWriter) == false)
		{
			UE_LOGF(LogJson, Warning, "UStructToFormattedObjectString - Unable to write out json");
		}
		JsonWriter->Close();
	}

	return RetString;
}

//----------------------------------------------------------------------//
// FMLAdapterSpaceDescription 
//----------------------------------------------------------------------//
FString FMLAdapterSpaceDescription::ToJson() const
{
	FString RetString;
	TSharedRef<FJsonObject> JsonObject = MakeShareable(new FJsonObject());

	bool bValid = false;
	//bValid = MapToJson(Data, JsonObject);
	bValid = PairArrayToJson(Data, JsonObject);

	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&RetString, /*Indent=*/0);

	if (FJsonSerializer::Serialize(JsonObject, JsonWriter) == false)
	{
		UE_LOGF(LogJson, Warning, "UStructToFormattedObjectString - Unable to write out json");
	}
	JsonWriter->Close();

	return RetString;
}
