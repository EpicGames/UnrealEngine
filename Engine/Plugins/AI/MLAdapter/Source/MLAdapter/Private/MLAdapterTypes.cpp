// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLAdapterTypes.h"
#include "GameFramework/Controller.h"
#include "GameFramework/Pawn.h"

DEFINE_LOG_CATEGORY(LogMLAdapter);

namespace FMLAdapter
{
	AController* ActorToController(AActor& Actor)
	{
		AController* AsController = Cast<AController>(&Actor);
		if (AsController == nullptr)
		{
			APawn* AsPawn = Cast<APawn>(&Actor);
			AsController = AsPawn ? AsPawn->GetController() : nullptr;
		}
		return AsController;
	}
}
