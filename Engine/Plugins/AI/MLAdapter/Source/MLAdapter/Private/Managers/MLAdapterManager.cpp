// Copyright Epic Games, Inc. All Rights Reserved.

#include "Managers/MLAdapterManager.h"
#include "CoreGlobals.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Engine/Engine.h"
#include "GameFramework/GameModeBase.h"
#include "MLAdapterTypes.h"
#include "MLAdapterAsync.h"
#include "Sessions/MLAdapterSession.h"
#include "MLAdapterJson.h"
#include "Agents/MLAdapterAgent.h"
#include "Sensors/MLAdapterSensor.h"
#include "MLAdapterSettings.h"
#include <string>
#if WITH_EDITORONLY_DATA
#include "Editor.h"
#include "Settings/LevelEditorPlaySettings.h"
#endif // WITH_EDITORONLY_DATA

#include "RPCWrapper/MsgPack.h"
#include "RPCWrapper/Server.h"

// engine AI support
#include "AI/NavigationSystemBase.h"


namespace
{
	FRPCServer* RPCServerInstance = nullptr;
	
	EMLAdapterServerMode GetServerModeForWorld(UWorld& World)
	{
		EMLAdapterServerMode Mode = EMLAdapterServerMode::Invalid;
		switch (World.GetNetMode())
		{
		case NM_Standalone:
		case NM_ListenServer:
			Mode = EMLAdapterServerMode::Standalone;
			break;
		case NM_DedicatedServer:
			Mode = EMLAdapterServerMode::Server;
			break;
		case NM_Client:
			Mode = EMLAdapterServerMode::Client;
			break;
		}
		ensure(Mode != EMLAdapterServerMode::Invalid);
		return Mode;
	}
}

void UMLAdapterManager::RecreateManagerInstance()
{
#if WITH_EDITOR
	if (UMLAdapterManager::IsReady())
	{
		UMLAdapterManager& PreviousInstance = UMLAdapterManager::Get();
		PreviousInstance.ConditionalBeginDestroy();
	}
#endif // WITH_EDITOR

	TSubclassOf<UMLAdapterManager> SettingsManagerClass = UMLAdapterSettings::GetManagerClass().Get();
	UClass* Class = SettingsManagerClass
		? SettingsManagerClass.Get()
		: UMLAdapterManager::StaticClass();

	UE_LOGF(LogMLAdapter, Log, "Creating MLAdapter manager of class %ls", *GetNameSafe(Class));

	UMLAdapterManager* NewInstance = NewObject<UMLAdapterManager>(GEngine, Class);
	check(NewInstance);
	NewInstance->AddToRoot();

	UMLAdapterManager::OnPostInit.Broadcast();
}

struct FManagerBootloader
{
	FManagerBootloader()
	{
		FCoreDelegates::GetOnPostEngineInit().AddLambda([this]()
			{
				UMLAdapterManager::RecreateManagerInstance();
			});
	}
};

static FManagerBootloader Loader;

//----------------------------------------------------------------------//
// 
//
// @todo we might want to look for a way to restrict users from creating UMLAdapterManager
// instances manually.
//----------------------------------------------------------------------//
UMLAdapterManager* UMLAdapterManager::ManagerInstance = nullptr;
UMLAdapterManager::FOnGenericEvent UMLAdapterManager::OnPostInit;

UMLAdapterManager::UMLAdapterManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCommonFunctionsAdded = false;
	bTickWorldManually = false;

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		ensure(ManagerInstance == nullptr);

		ManagerInstance = this;
	}
}

void UMLAdapterManager::PostInitProperties()
{
	Super::PostInitProperties();
	
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		Librarian.GatherClasses();

		BindToDelegates();

		// if there's any world present create the RPC server
		UWorld* World = 
#if WITH_EDITOR
			GIsEditor ? GWorld :
#endif // WITH_EDITOR
			(GEngine->GetWorldContexts().Num() > 0 ? GEngine->GetWorldContexts()[0].World() : nullptr);
		
		OnPostWorldInit(World, UWorld::InitializationValues());
	}
}

void UMLAdapterManager::BeginDestroy()
{
	SetSession(nullptr);
	if (ManagerInstance == this)
	{
		//SetManualWorldTickEnabled(false);
		ManagerInstance = nullptr;
	}
	StopServer();
	CleanUpDelegates();
	Super::BeginDestroy();
}

void UMLAdapterManager::StopServer()
{
	if (RPCServerInstance)
	{
		UE_LOGF(LogMLAdapter, Log, "Stopping RPC server.");
		RPCServerInstance->stop();
		delete RPCServerInstance;
		RPCServerInstance = nullptr;
	}
	CurrentFunctionMode = EMLAdapterServerMode::Invalid;
}

void UMLAdapterManager::StartServer(uint16 Port, EMLAdapterServerMode InMode, uint16 ServerThreads)
{
	StopServer();
	
	RequestedFunctionMode = InMode;
	ServerThreads = FMath::Max< uint16>(1, ServerThreads);

	EMLAdapterServerMode NewMode = InMode;
	if (InMode == EMLAdapterServerMode::Invalid || InMode == EMLAdapterServerMode::AutoDetect)
	{
		if (LastActiveWorld)
		{
			NewMode = GetServerModeForWorld(*LastActiveWorld);
		}
		else if (GIsEditor || (GIsClient && GIsServer))
		{
			NewMode = EMLAdapterServerMode::Standalone;
		}
		else if (GIsClient)
		{
			NewMode = EMLAdapterServerMode::Client;
		}
		else
		{
			NewMode = EMLAdapterServerMode::Server;
		}
	}

	UE_LOGF(LogMLAdapter, Log, "Starting RPC server on port %d.", Port);
	RPCServerInstance = new FRPCServer(Port);
	CurrentPort = Port;
	check(RPCServerInstance);

	bCommonFunctionsAdded = false;

	CurrentFunctionMode = NewMode;
	switch (NewMode)
	{
	case EMLAdapterServerMode::Client:
		ConfigureAsClient(*RPCServerInstance);
		break;
	case EMLAdapterServerMode::Server:
		ConfigureAsServer(*RPCServerInstance);
		break;
	default:
		ConfigureAsClient(*RPCServerInstance);
		ConfigureAsServer(*RPCServerInstance);
		break;
	}
	CurrentServerThreads = ServerThreads;
	RPCServerInstance->async_run(ServerThreads);
}

void UMLAdapterManager::ConfigureAsStandalone(FRPCServer& Server)
{
	ConfigureAsServer(Server);
	ConfigureAsClient(Server);
}

void UMLAdapterManager::EnsureAISystemPresence(UWorld& World)
{
	if (World.GetAISystem())
	{
		return;
	}

	UAISystemBase* AISystem = World.CreateAISystem();
	// it's possible the world is configured to not have AI system. Not sure what to do in such a a case
	ensure(AISystem);
}

void UMLAdapterManager::EnsureNavigationSystemPresence(UWorld& World)
{
	if (World.GetNavigationSystem())
	{
		return;
	}
	
	FNavigationSystem::AddNavigationSystemToWorld(World);	
	// it's possible the world is configured to not have a nav system. Not sure what to do in such a a case
	ensure(World.GetNavigationSystem());
}

bool UMLAdapterManager::IsRunning() const
{
	return (RPCServerInstance != nullptr);
}

TStatId UMLAdapterManager::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UMLAdapterManager, STATGROUP_Tickables);
}

ETickableTickType UMLAdapterManager::GetTickableTickType() const 
{ 
	return HasAnyFlags(RF_ClassDefaultObject) ? ETickableTickType::Never : ETickableTickType::Always;
}

bool UMLAdapterManager::IsTickable() const 
{ 
	return (HasAnyFlags(RF_ClassDefaultObject) == false); 
}

void UMLAdapterManager::Tick(float DeltaTime)
{
	if (IsWorldRealTime() || StepsRequested > 0)
	{
		if (Session)
		{
			DeltaTime = IsWorldRealTime() ? DeltaTime : (1.f / WorldFPS);
			Session->Tick(DeltaTime);
		}
		--StepsRequested;
	}
}

UMLAdapterSession* UMLAdapterManager::CreateNewSession()
{
	UClass* Class = UMLAdapterSettings::GetSessionClass().Get() 
		? UMLAdapterSettings::GetSessionClass().Get()
		: UMLAdapterSession::StaticClass();

	UE_LOGF(LogMLAdapter, Log, "Creating MLAdapter session of class %ls", *GetNameSafe(Class));

	UMLAdapterSession* NewSession = FMLAdapter::NewObject<UMLAdapterSession>(this, Class);
	check(NewSession);
	NewSession->SetWorld(LastActiveWorld);
	// some config
	return NewSession;
}

void UMLAdapterManager::SetSession(UMLAdapterSession* NewSession)
{
	if (Session != nullptr && (NewSession == nullptr || Session != NewSession))
	{
		Session->Close();
		// clear the flag to let GC remove the object
		Session->ClearInternalFlags(EInternalObjectFlags::Async);
		Session = nullptr;
	}
	Session = NewSession;
	if (Session)
	{
		// we're going to be using this object in a async manner, so we need to 
		// mark it appropriately. This will make GC ignore this object until we 
		// clear the flag. 
		Session->SetInternalFlags(EInternalObjectFlags::Async);
		Session->Open();
	}

	OnCurrentSessionChanged.Broadcast();
}

void UMLAdapterManager::CloseSession(UMLAdapterSession& InSession)
{
	// @todo temporary implementation, will change with multi-session support
	if (&InSession == Session)
	{
		SetSession(nullptr);
	}
}

UMLAdapterSession& UMLAdapterManager::GetSession()
{
	if (Session == nullptr)
	{
		SetSession(CreateNewSession());
	}
	check(Session);
	return *Session;
}

void UMLAdapterManager::BindToDelegates()
{	
	// Commented out possible other useful delegates
	//	FCoreDelegates::GameStateClassChanged;
	//	FCoreDelegates::TSConfigReadyForUse();
	//  FWorldDelegates::OnPostWorldCreation;
	//	FWorldDelegates::OnPreWorldInitialization; 

	OnPostWorldInitializationHandle = FWorldDelegates::OnPostWorldInitialization.AddUObject(this, &UMLAdapterManager::OnPostWorldInit);
	OnWorldCleanupHandle = FWorldDelegates::OnWorldCleanup.AddUObject(this, &UMLAdapterManager::OnWorldCleanup);

	//FGameDelegates
	OnGameModeInitializedHandle = FGameModeEvents::OnGameModeInitializedEvent().AddUObject(this, &UMLAdapterManager::OnGameModeInitialized);
	// FGameModePreLoginEvent GameModePreLoginEvent;
	OnGameModePostLoginHandle = FGameModeEvents::OnGameModePostLoginEvent().AddUObject(this, &UMLAdapterManager::OnGameModePostLogin);
	// FGameModeLogoutEvent GameModeLogoutEvent;
	OnGameModeMatchStateSetHandle = FGameModeEvents::OnGameModeMatchStateSetEvent().AddUObject(this, &UMLAdapterManager::OnGameModeMatchStateSet);

#if WITH_EDITORONLY_DATA
	BeginPIEHandle = FEditorDelegates::BeginPIE.AddUObject(this, &UMLAdapterManager::OnBeginPIE);
	EndPIEHandle = FEditorDelegates::EndPIE.AddUObject(this, &UMLAdapterManager::OnEndPIE);
#endif // WITH_EDITORONLY_DATA
}

void UMLAdapterManager::CleanUpDelegates()
{
	FWorldDelegates::OnPostWorldInitialization.Remove(OnPostWorldInitializationHandle);
	FWorldDelegates::OnWorldCleanup.Remove(OnWorldCleanupHandle);
	FGameModeEvents::OnGameModeInitializedEvent().Remove(OnGameModeInitializedHandle);
	FGameModeEvents::OnGameModePostLoginEvent().Remove(OnGameModePostLoginHandle);
	FGameModeEvents::OnGameModeMatchStateSetEvent().Remove(OnGameModeMatchStateSetHandle);

#if WITH_EDITORONLY_DATA
	FEditorDelegates::BeginPIE.Remove(BeginPIEHandle);
	FEditorDelegates::EndPIE.Remove(EndPIEHandle);
#endif // WITH_EDITORONLY_DATA
}

bool UMLAdapterManager::ShouldInitForWorld(const UWorld& World) const
{
	if (!World.IsGameWorld())
	{
		return false;
	}

#if WITH_EDITOR
	return World.WorldType == EWorldType::Game
		|| (World.WorldType == EWorldType::PIE && World.HasAnyFlags(RF_WasLoaded));
#else
	return true;
#endif // WITH_EDITOR
}

void UMLAdapterManager::OnPostWorldInit(UWorld* World, const UWorld::InitializationValues)
{
	if (World && ShouldInitForWorld(*World))
	{
		LastActiveWorld = World;
		if ((RequestedFunctionMode == EMLAdapterServerMode::Invalid || RequestedFunctionMode == EMLAdapterServerMode::AutoDetect)
			&& GetServerModeForWorld(*World) != CurrentFunctionMode)
		{
			// restart the RPC server. Note that this will kick all the currently connected agents
			uint16 Port = UMLAdapterSettings::GetDefaultRPCServerPort();
			FParse::Value(FCommandLine::Get(), TEXT("MLAdapterPort="), Port);
			StartServer(Port, GetServerModeForWorld(*World), CurrentServerThreads);
		}

		if (HasSession())
		{
			GetSession().OnPostWorldInit(*World);
		}
	}
}

void UMLAdapterManager::OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	// no need to remove, the World is going away
	if (World && World->IsGameWorld())
	{
		if (World == LastActiveWorld)
		{
			LastActiveWorld = nullptr;
		}

		if (HasSession())
		{
			CloseSession(GetSession());
		}
	}
}

void UMLAdapterManager::OnBeginPIE(const bool bIsSimulating)
{

}

void UMLAdapterManager::OnEndPIE(const bool bIsSimulating)
{

}

void UMLAdapterManager::OnGameModeInitialized(AGameModeBase* GameMode)
{
	if (GameMode && HasSession())
	{
		GetSession().OnGameModeInitialized(*GameMode);
	}
}

void UMLAdapterManager::OnGameModePostLogin(AGameModeBase* GameMode, APlayerController* NewPlayer)
{
	if (GameMode && NewPlayer && HasSession())
	{
		GetSession().OnGameModePostLogin(GameMode, NewPlayer);
	}
}

void UMLAdapterManager::OnGameModeMatchStateSet(FName MatchState)
{
	if (HasSession())
	{
		GetSession().OnGameModeMatchStateSet(MatchState);
	}
}

void UMLAdapterManager::ResetWorld()
{
#if WITH_EDITORONLY_DATA
	// Reset logic won't work if the game thread is paused so we need to resume it
	if (GIntraFrameDebuggingGameThread)
	{
		GIntraFrameDebuggingGameThread = false;
	}
#endif // WITH_EDITORONLY_DATA

	if (LastActiveWorld)
	{
		AGameModeBase* GameMode = LastActiveWorld->GetAuthGameMode<AGameModeBase>();
		if (GameMode)
		{
			GameMode->ResetLevel();
		}
	}

	if (HasSession())
	{
		GetSession().ResetWorld();
	}
}

void UMLAdapterManager::SetManualWorldTickEnabled(bool bEnable)
{
	bTickWorldManually = bEnable;
	if (Session)
	{
		Session->SetManualWorldTickEnabled(bEnable);
	}
}

bool UMLAdapterManager::Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return false;
	}

	if (FParse::Command(&Cmd, TEXT("mladapter_session_stop")))
	{
		SetSession(nullptr);
		return true;
	}

	return false;
}

namespace FMLAdapterConsoleCommands
{
	struct FHelper
	{
		static void RestartServer(const TArray<FString>& Args, UWorld*)
		{
			UMLAdapterManager& Manager = UMLAdapterManager::Get();
			uint16 Port = UMLAdapterSettings::GetDefaultRPCServerPort();
			FParse::Value(FCommandLine::Get(), TEXT("MLAdapterPort="), Port);
			if (Args.Num() > 0)
			{
				Port = uint16(TCString<TCHAR>::Atoi(*Args[0]));
			}
			Manager.StartServer(Port, Manager.CurrentFunctionMode, Manager.CurrentServerThreads);
		}
	};

	FAutoConsoleCommand StopSession(TEXT("mladapter.session.stop"), TEXT(""), FConsoleCommandDelegate::CreateLambda([]()
	{
		UMLAdapterManager::Get().SetSession(nullptr);
	}));

	FAutoConsoleCommandWithWorldAndArgs RestartServer(TEXT("mladapter.server.restart")
		, TEXT("restarts the MLAdapter RPC server, optionally changing the port the server is listening at. Use: mladapter.server.restart [port]")
		, FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(FHelper::RestartServer));
}
