// Copyright Epic Games, Inc. All Rights Reserved.

#include "Managers/MLAdapterManager.h"
#include "MLAdapterTypes.h"
#include "MLAdapterAsync.h"
#include "Sessions/MLAdapterSession.h"
#include "MLAdapterScribe.h"
#include "MLAdapterSpace.h"
#include "Agents/MLAdapterAgent.h"
#include "Actuators/MLAdapterActuator.h"
#include "Engine/GameEngine.h"

#include "RPCWrapper/Server.h"

#define REPORT_NOT_IMPLEMENTED() rpc::this_handler().respond_error("Not implemented yet")
#define CHECK_AGENTID(AgentID) { \
	if (HasSession() == false) \
	{ \
		rpc::this_handler().respond_error("No active session"); \
	}\
	if (GetSession().GetAgent(AgentID) == nullptr) \
	{ \
		rpc::this_handler().respond_error(std::make_tuple("No Agent of ID", AgentID)); \
	} \
} \


void UMLAdapterManager::AddCommonFunctions(FRPCServer& Server)
{
	if (bCommonFunctionsAdded)
	{
		return;
	}

#if WITH_RPCLIB
	Server.bind("list_functions", &FMLAdapterScribe::ListFunctions);
	Librarian.AddRPCFunctionDescription(TEXT("list_functions"), TEXT("(), Lists all functions available through RPC"));

	Server.bind("get_description", [](std::string const& ElementName) {
		return FMLAdapterScribe::GetDescription(ElementName);
	});
	Librarian.AddRPCFunctionDescription(TEXT("get_description"), TEXT("(string ElementName), Describes given element"));

	Server.bind("list_sensor_types", &FMLAdapterScribe::ListSensorTypes);
	Librarian.AddRPCFunctionDescription(TEXT("list_sensor_types"), TEXT("(), Lists all sensor types available to agents. Note that some of sensors might not make sense in a given environment (like reading keyboard in an mouse-only game)."));

	Server.bind("list_actuator_types", &FMLAdapterScribe::ListActuatorTypes);
	Librarian.AddRPCFunctionDescription(TEXT("list_actuator_types"), TEXT("(), Lists all actuator types available to agents. Note that some of actuators might not make sense in a given environment (like faking keyboard actions in an mouse-only game)."));

	Server.bind("ping", []() { return true; });
	Librarian.AddRPCFunctionDescription(TEXT("ping"), TEXT("(), Checks if the RPC server is still alive and responding."));

	Server.bind("get_name", []() {
		return std::string(TCHAR_TO_UTF8(GInternalProjectName));
	});
	Librarian.AddRPCFunctionDescription(TEXT("get_name"), TEXT("(), Fetches a human-readable identifier of the environment the external client is connected to."));

	Server.bind("is_finished", [this](FMLAdapter::FAgentID AgentID) {
		return HasSession() == false || GetSession().IsDone() || GetSession().GetAgent(AgentID) == nullptr 
			|| GetSession().GetAgent(AgentID)->IsDone();
	});
	Librarian.AddRPCFunctionDescription(TEXT("is_finished"), TEXT("(agent_id), Checks if the game/simulation/episode is done for given agent_id."));

	Server.bind("exit", []() {
		FPlatformMisc::RequestExit(/*bForce=*/false);
	});
	Librarian.AddRPCFunctionDescription(TEXT("exit"), TEXT("(), Closes the UnrealEngine instance."));
	
	Server.bind("batch_is_finished", [this](std::vector<FMLAdapter::FAgentID> AgentIDs) {
		std::vector<bool> Results;
		if (HasSession() == false || GetSession().IsDone())
		{
			for (int Index = 0; Index < int(AgentIDs.size()); ++Index)
			{
				Results.push_back(true);
			}
		}
		else
		{
			UMLAdapterSession& SessionInstance = GetSession();
			for (const FMLAdapter::FAgentID& AgentID : AgentIDs)
			{
				UMLAdapterAgent* Agent = SessionInstance.GetAgent(AgentID);
				Results.push_back(Agent == nullptr || Agent->IsDone());
			}
		}
		return Results;
	});
	Librarian.AddRPCFunctionDescription(TEXT("batch_is_finished"), TEXT("(), Multi-agent version of is_finished"));
#endif // WITH_RPCLIB

	bCommonFunctionsAdded = true;
}

void UMLAdapterManager::ConfigureAsClient(FRPCServer& Server)
{
	UE_LOGF(LogMLAdapter, Log, "\tconfiguring as client");

	AddCommonFunctions(Server);

#if WITH_RPCLIB
	Server.bind("add_agent", [this]() {
		return CallOnGameThread<FMLAdapter::FAgentID>([this]()
		{
			return GetSession().AddAgent();
		});
	});
	Librarian.AddRPCFunctionDescription(TEXT("add_agent"), TEXT("Adds a default agent for current environment. Returns added agent's ID if successful, uint(-1) if failed."));

	Server.bind("get_agent_config", [this](FMLAdapter::FAgentID AgentID) {
		CHECK_AGENTID(AgentID);
		const FMLAdapterAgentConfig& Config = GetSession().GetAgent(AgentID)->GetConfig();
		return FSTRING_TO_STD(FMLAdapter::StructToJsonString(Config));
	});
	Librarian.AddRPCFunctionDescription(TEXT("get_agent_config"), TEXT("(uint AgentID), Retrieves given agent's config in JSON formatted string"));

	Server.bind("act", [this](FMLAdapter::FAgentID AgentID, std::vector<float> ValueStream) {
		CHECK_AGENTID(AgentID);
		UMLAdapterAgent* Agent = GetSession().GetAgent(AgentID);
		check(Agent);

		const uint8* DataPtr = (const uint8*)ValueStream.data();
		TArray<uint8> Buffer;
		Buffer.Append(DataPtr, ValueStream.size() * sizeof(float));
		FMLAdapterMemoryReader Reader(Buffer);
		Agent->DigestActions(Reader);
	});
	Librarian.AddRPCFunctionDescription(TEXT("act"), TEXT("(uint agent_id, list actions), Distributes the given values array amongst all the actuators, based on actions_space."));

	Server.bind("batch_act", [this](std::vector<FMLAdapter::FAgentID> AgentIDs, std::vector<std::vector<float>> ValueStreams) {
		if (HasSession() == false)
		{
			rpc::this_handler().respond_error("No active session");
		}

		for (int Index = 0; Index < int(AgentIDs.size()); ++Index)
		{
			UMLAdapterAgent* Agent = GetSession().GetAgent(AgentIDs[Index]);
			if (Agent)
			{
				const uint8* DataPtr = (const uint8*)ValueStreams[Index].data();
				TArray<uint8> Buffer;
				Buffer.Append(DataPtr, ValueStreams[Index].size() * sizeof(float));
				FMLAdapterMemoryReader Reader(Buffer);
				Agent->DigestActions(Reader);
			}
		}
	});
	Librarian.AddRPCFunctionDescription(TEXT(""), TEXT("A multi-agent version of \'act\' function"));

	Server.bind("get_observations", [this](FMLAdapter::FAgentID AgentID) {
		std::vector<float> Values;
		if (HasSession() && GetSession().GetAgent(AgentID))
		{
			TArray<uint8> Buffer;
			FMLAdapterMemoryWriter Writer(Buffer);
			//Observations.TimeStamp = GetSession().GetTimestamp();
			GetSession().GetAgent(AgentID)->GetObservations(Writer);

			const float* DataPtr = (float*)Buffer.GetData();
			Values.assign(DataPtr, DataPtr + Buffer.Num() / sizeof(float));
		}
		return Values;
	});
	Librarian.AddRPCFunctionDescription(TEXT("get_observations"), TEXT("(uint agent_id), fetches all the information gathered by given agent's sensors. Result matches observations_space"));

	Server.bind("batch_get_observations", [this](std::vector<FMLAdapter::FAgentID> AgentIDs) {
		std::vector<std::vector<float>> Values;
		if (HasSession())
		{
			Values.resize(AgentIDs.size());
			for (int Index = 0; Index < int(AgentIDs.size()); ++Index)
			{
				UMLAdapterAgent* Agent = GetSession().GetAgent(AgentIDs[Index]);
				if (Agent)
				{
					TArray<uint8> Buffer;
					FMLAdapterMemoryWriter Writer(Buffer);
					Agent->GetObservations(Writer);

					const float* DataPtr = (float*)Buffer.GetData();
					Values[Index].assign(DataPtr, DataPtr + Buffer.Num() / sizeof(float));
				}
			}
		}
		return Values;
	});
	Librarian.AddRPCFunctionDescription(TEXT("batch_get_observations"), TEXT("Multi-agent version of 'get_observations'"));

	Server.bind("get_recent_agent", [this]() {
		return HasSession() ? FMLAdapter::FAgentID(GetSession().GetAgentsCount() - 1) : FMLAdapter::InvalidAgentID;
	});
	Librarian.AddRPCFunctionDescription(TEXT("get_recent_agent"), TEXT("(), Fetches ID of the most recently created agent."));

	Server.bind("get_reward", [this](FMLAdapter::FAgentID AgentID) {
		CHECK_AGENTID(AgentID);
		return GetSession().GetAgent(AgentID)->GetReward();
	});
	Librarian.AddRPCFunctionDescription(TEXT("get_reward"), TEXT("(uint agent_id), Fetch current reward for given Agent."));

	Server.bind("batch_get_rewards", [this](std::vector<FMLAdapter::FAgentID> AgentIDs) {
		if (HasSession() == false)
		{
			rpc::this_handler().respond_error("No active session");
		}
		std::vector<float> Rewards;
		for (const FMLAdapter::FAgentID& AgentID : AgentIDs)
		{
			UMLAdapterAgent* Agent = GetSession().GetAgent(AgentID);
			Rewards.push_back(Agent ? Agent->GetReward() : 0.f);
		}

		return Rewards;
	});
	Librarian.AddRPCFunctionDescription(TEXT("batch_get_rewards"), TEXT("(), Multi-agent version of 'get_rewards'."));

	Server.bind("desc_action_space", [this](FMLAdapter::FAgentID AgentID) {
		return CallOnGameThread<std::string>([this, AgentID]()
		{
			CHECK_AGENTID(AgentID);
			FMLAdapterSpaceDescription SpaceDesc;
			GetSession().GetAgent(AgentID)->GetActionSpaceDescription(SpaceDesc);
			return FSTRING_TO_STD(SpaceDesc.ToJson());
		});
	});
	Librarian.AddRPCFunctionDescription(TEXT("desc_action_space"), TEXT("(uint agent_id), Fetches actions space desction for given agent"));

	// we're sending this call to game thread since if it's called right after 
	// "configure_agent" then this call will fetch pre-config state due to agent 
	// configuration being performed on the game thread
	Server.bind("desc_observation_space", [this](FMLAdapter::FAgentID AgentID) {
		return CallOnGameThread<std::string>([this, AgentID]()
		{
			CHECK_AGENTID(AgentID);
			FMLAdapterSpaceDescription SpaceDesc;
			GetSession().GetAgent(AgentID)->GetObservationSpaceDescription(SpaceDesc);
			return FSTRING_TO_STD(SpaceDesc.ToJson());
		});
	}); 
	Librarian.AddRPCFunctionDescription(TEXT("desc_observation_space"), TEXT("(uint agent_id), Fetches observations space desction for given agent"));
		
	Server.bind("reset", []() {
		CallOnGameThread<void>([]()
		{
			UMLAdapterManager::Get().ResetWorld();
		});
	});
	Librarian.AddRPCFunctionDescription(TEXT("reset"), TEXT("(), Lets the MLAdapter manager know that the environments should be reset. The details of how this call is handles heavily depends on the environment itself."));

	Server.bind("disconnect", [this](FMLAdapter::FAgentID AgentID) {
		CHECK_AGENTID(AgentID);
		GetSession().RemoveAgent(AgentID);
	});
	Librarian.AddRPCFunctionDescription(TEXT(""), TEXT("(uint agent_id), Lets the MLAdapter session know that given agent will not continue and is to be removed from the session."));

	// calling this means we're done messing up with the agent (configuring and all) 
	// and we're ready to roll
	Server.bind("configure_agent", [this](FMLAdapter::FAgentID AgentID, std::string const& JsonConfigString) {

		if (HasSession() == false)
		{
			rpc::this_handler().respond_error("No active session");
			return;
		}
		else if (GetSession().GetAgent(AgentID) == nullptr)
		{
			rpc::this_handler().respond_error(std::make_tuple("No Agent of ID", AgentID));
			return;
		}

		FMLAdapterAgentConfig Config;
		FMLAdapter::JsonStringToStruct(FString(JsonConfigString.c_str()), Config);

		CallOnGameThread<void>([this, Config, AgentID]()
		{
			GetSession().GetAgent(AgentID)->Configure(Config);
		});
	});
	Librarian.AddRPCFunctionDescription(TEXT("configure_agent"), TEXT("(uint agent_id, string json_config), Configures given agent based on json_config. Will throw an exception if given agent doesn't exist."));

	// combines 'add' and 'configure' agent
	Server.bind("create_agent", [this](std::string const& JsonConfigString) {
		FMLAdapterAgentConfig Config;
		FMLAdapter::JsonStringToStruct(FString(JsonConfigString.c_str()), Config);

		return CallOnGameThread<FMLAdapter::FAgentID>([this, Config]()
		{
			return GetSession().AddAgent(Config);
		});
	});
	Librarian.AddRPCFunctionDescription(TEXT("create_agent"), TEXT("(), Creates a new agent and returns its agent_id."));

	Server.bind("is_agent_ready", [this](FMLAdapter::FAgentID AgentID) {
		return HasSession() && GetSession().IsAgentReady(AgentID);
	});
	Librarian.AddRPCFunctionDescription(TEXT("is_agent_ready"), TEXT("(uint agent_id), Returns 'true' if given agent is ready to play, including having an avatar"));

	Server.bind("is_ready", [this]() {
		return CallOnGameThread<bool>([this]()
		{
			return HasSession() && GetSession().IsReady();
		});
	});
	Librarian.AddRPCFunctionDescription(TEXT("is_ready"), TEXT("(), return whether the session is ready to go, i.e. whether the simulation has loaded and started."));
#endif // WITH_RPCLIB

	if (Session)
	{
		Session->ConfigureAsClient();
	}
	OnAddClientFunctions.Broadcast(Server);
}