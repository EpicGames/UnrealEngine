// Copyright Epic Games, Inc. All Rights Reserved.

#include "Managers/MLAdapterManager.h"
#include "CoreGlobals.h"
#include "Misc/CoreDelegates.h"
#include "Engine/Engine.h"
#include "GameFramework/GameModeBase.h"
#include "MLAdapterTypes.h"
#include "MLAdapterAsync.h"
#include "Sessions/MLAdapterSession.h"
#include "MLAdapterJson.h"
#include "Agents/MLAdapterAgent.h"
#include "Sensors/MLAdapterSensor.h"
#include "MLAdapterSettings.h"
#include <string>
#if WITH_EDITORONLY_DATA
#include "Editor.h"
#endif // WITH_EDITORONLY_DATA

#include "RPCWrapper/Server.h"


void UMLAdapterManager::ConfigureAsServer(FRPCServer& Server)
{
	UE_LOGF(LogMLAdapter, Log, "\tconfiguring as server");

	AddCommonFunctions(Server);

#if WITH_RPCLIB
	Server.bind("enable_manual_world_tick", [this](bool bEnable) {
		SetManualWorldTickEnabled(bEnable);
	});
	Librarian.AddRPCFunctionDescription(TEXT("enable_manual_world_tick"), TEXT("(bool bEnable), Controls whether the world is running real time or it\'s being ticked manually with calls to \'step\' or \'request_world_tick\' functions. Default is \'real time\'."));

	Server.bind("request_world_tick", [this](int32 TickCount, bool bWaitForWorldTick) {
		if (bTickWorldManually == false)
		{
			return;
		}
		StepsRequested = TickCount;
		while (bWaitForWorldTick && StepsRequested > 0)
		{
			FPlatformProcess::Sleep(0.f);
		}
		return;
	});
	Librarian.AddRPCFunctionDescription(TEXT("request_world_tick"), TEXT("(int TickCount, bool bWaitForWorldTick), Requests a TickCount world ticks. This has meaning only if \'enable_manual_world_tick(true)\' has been called prior to this function. If bWaitForWorldTick is true then the call will not return until the world has been ticked required number of times"));

	Server.bind("enable_action_duration", [this](FMLAdapter::FAgentID AgentID, bool bEnableActionDuration, float DurationSeconds) {
		if (Session)
		{
			Session->EnableActionDuration(AgentID, bEnableActionDuration, DurationSeconds);
		}
	});
	Librarian.AddRPCFunctionDescription(TEXT("enable_action_duration"), TEXT("(uint AgentID, bool bEnableActionDuration, float DurationSeconds), Enable/disable the action durations on the agent with the specified time duration in seconds."));

	Server.bind("wait_for_action_duration", [this](FMLAdapter::FAgentID AgentID) {
		if (Session)
		{
			while (!Session->TryResetActionDuration(AgentID))
			{
			}
		}
	});
	Librarian.AddRPCFunctionDescription(TEXT("wait_for_action_duration"), TEXT("(uint AgentID), Wait for the action duration to elapse for the agent. Only works if \'enable_action_duration\' has been called previously."));

	Server.bind("close_session", [this]() { UMLAdapterManager::Get().SetSession(nullptr); });
	Librarian.AddRPCFunctionDescription(TEXT("close_session"), TEXT("(), shuts down the current session (along with all the agents)."));
#endif // WITH_RPCLIB

	if (Session)
	{
		Session->ConfigureAsServer();
	}
	OnAddServerFunctions.Broadcast(Server);
}