// Copyright Epic Games, Inc. All Rights Reserved.

#include "Managers/MLAdapterNoRPCManager.h"
#include "Sessions/MLAdapterSession.h"

void UMLAdapterNoRPCManager::OnPostWorldInit(UWorld* World, const UWorld::InitializationValues)
{
	if (World && ShouldInitForWorld(*World))
	{
		LastActiveWorld = World;

		// GetSession will create the initial session
		GetSession().OnPostWorldInit(*World);

		// AddAgent will add the default agent from the settings
		FMLAdapter::FAgentID AgentID = GetSession().AddAgent();
	}
}
