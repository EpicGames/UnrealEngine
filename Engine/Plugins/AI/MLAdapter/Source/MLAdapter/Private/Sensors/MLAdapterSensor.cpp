// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sensors/MLAdapterSensor.h"
#include "Agents/MLAdapterAgent.h"
#include "GameFramework/Pawn.h"
#include "MLAdapterTypes.h"
#include "Managers/MLAdapterManager.h"


namespace
{
	uint32 NextSensorID = FMLAdapter::InvalidSensorID + 1;
}

UMLAdapterSensor::UMLAdapterSensor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	AgentID = FMLAdapter::InvalidAgentID;
	bRequiresPawn = true;
	bIsPolling = true;

	TickPolicy = EMLAdapterTickPolicy::EveryTick; 

	ElementID = FMLAdapter::InvalidSensorID;
}

void UMLAdapterSensor::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		if (GetClass()->HasAnyClassFlags(CLASS_Abstract) == false)
		{
			ElementID = NextSensorID++;
		}
	}
	else
	{
		const UMLAdapterSensor* CDO = GetDefault<UMLAdapterSensor>(GetClass());
		check(CDO);
		ElementID = CDO->ElementID;
	}
}

void UMLAdapterSensor::Configure(const TMap<FName, FString>& Params)
{
	Super::Configure(Params);

	const FName NAME_EveryTick = TEXT("tick_every_frame");
	const FName NAME_EveryNTicks = TEXT("tick_every_n_frames");
	const FName NAME_EveryXFrames = TEXT("tick_every_x_seconds");

	for (auto KeyValue : Params)
	{
		if (KeyValue.Key == NAME_EveryTick)
		{
			TickPolicy = EMLAdapterTickPolicy::EveryTick;
		}
		else if (KeyValue.Key == NAME_EveryNTicks)
		{
			TickPolicy = EMLAdapterTickPolicy::EveryNTicks;
			ensure(KeyValue.Value.Len() > 0);
			TickEvery.Ticks = FCString::Atoi(*KeyValue.Value);
		}
		else if (KeyValue.Key == NAME_EveryXFrames)
		{
			TickPolicy = EMLAdapterTickPolicy::EveryXSeconds;
			ensure(KeyValue.Value.Len() > 0);
			TickEvery.Ticks = FCString::Atof(*KeyValue.Value);
		}
	}
}

void UMLAdapterSensor::OnAvatarSet(AActor* Avatar)
{
	// kick off first sensing to populate observation data
	SenseImpl(0.f);
	AccumulatedTicks = 0;
	AccumulatedSeconds = 0.f;
}

void UMLAdapterSensor::Sense(const float DeltaTime)
{
	++AccumulatedTicks;
	AccumulatedSeconds += DeltaTime;

	bool bTick = false;
	switch (TickPolicy)
	{
	case EMLAdapterTickPolicy::EveryXSeconds:
		bTick = (AccumulatedSeconds >= TickEvery.Seconds);
		break;
	case EMLAdapterTickPolicy::EveryNTicks:
		bTick = (AccumulatedTicks >= TickEvery.Ticks);
		break;
	case EMLAdapterTickPolicy::Never:
		bTick = false;
		break;
	default:
		bTick = true;
		break;
	}

	if (bTick)
	{
		SenseImpl(AccumulatedSeconds);
		AccumulatedTicks = 0;
		AccumulatedSeconds = 0.f;
	}
}

bool UMLAdapterSensor::IsConfiguredForAgent(const UMLAdapterAgent& Agent) const
{
	return AgentID == Agent.GetAgentID();
}

bool UMLAdapterSensor::ConfigureForAgent(UMLAdapterAgent& Agent)
{
	AgentID = Agent.GetAgentID();
	return true;
}

const UMLAdapterAgent& UMLAdapterSensor::GetAgent() const
{
	return *CastChecked<UMLAdapterAgent>(GetOuter());
}

void UMLAdapterSensor::OnPawnChanged(APawn* OldPawn, APawn* NewPawn)
{
	if (OldPawn)
	{
		ClearPawn(*OldPawn);
	}
	if (NewPawn)
	{
		OnAvatarSet(NewPawn);
	}
}

void UMLAdapterSensor::ClearPawn(APawn& InPawn)
{
	// Can be overridden in derived classes
}
