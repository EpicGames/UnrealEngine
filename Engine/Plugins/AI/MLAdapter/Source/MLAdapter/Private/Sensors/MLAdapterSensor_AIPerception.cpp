// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sensors/MLAdapterSensor_AIPerception.h"
#include "Agents/MLAdapterAgent.h"
#include "GameFramework/Controller.h"
#include "Perception/AIPerceptionComponent.h"
#include "Debug/DebugHelpers.h"
#include "Managers/MLAdapterManager.h"
#include "Perception/AISenseConfig_Sight.h"
#include "MLAdapterSpace.h"

#include "GameFramework/PlayerController.h"

//----------------------------------------------------------------------//
//  UMLAdapterSensor_AIPerception
//----------------------------------------------------------------------//
UMLAdapterSensor_AIPerception::UMLAdapterSensor_AIPerception(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	TargetsToSenseCount = 1;
	TargetsSortType = ESortType::Distance;
	PeripheralVisionAngleDegrees = 60.f;
	MaxStimulusAge = 0.6f;
}

void UMLAdapterSensor_AIPerception::Configure(const TMap<FName, FString>& Params)
{
	Super::Configure(Params);

	const FName NAME_Count = TEXT("count");
	const FString* CountValue = Params.Find(NAME_Count);
	if (CountValue != nullptr)
	{
		TargetsToSenseCount = FMath::Max(FCString::Atoi((TCHAR*)CountValue), 1);
	}

	const FName NAME_Sort = TEXT("sort");
	const FString* SortValue = Params.Find(NAME_Sort);
	if (SortValue != nullptr)
	{
		TargetsSortType = (*SortValue == TEXT("in_front")) ? ESortType::InFrontness : ESortType::Distance;
	}

	const FName NAME_Mode = TEXT("mode");
	const FString* ModeValue = Params.Find(NAME_Mode);
	if (ModeValue != nullptr)
	{
		bVectorMode = (ModeValue->Find(TEXT("vector")) != INDEX_NONE);
	}

	const FName NAME_PeripheralAngle = TEXT("peripheral_angle");
	const FString* PeripheralAngleValue = Params.Find(NAME_PeripheralAngle);
	if (PeripheralAngleValue != nullptr)
	{
		PeripheralVisionAngleDegrees = FMath::Max(FCString::Atof((TCHAR*)PeripheralAngleValue), 1.f);
	}

	const FName NAME_MaxAge = TEXT("max_age");
	const FString* MaxAgeValue = Params.Find(NAME_MaxAge);
	if (MaxAgeValue != nullptr)
	{
		MaxStimulusAge = FMath::Max(FCString::Atof((TCHAR*)MaxAgeValue), 0.001f);
	}

	UpdateSpaceDef();
}

TSharedPtr<FMLAdapter::FSpace> UMLAdapterSensor_AIPerception::ConstructSpaceDef() const
{
	FMLAdapter::FSpace* Result = nullptr;
	// FVector + Distance + ID -> 5
	// FRotator.Yaw + FRotator.Pitch + Distance + ID -> 4
	const uint32 ValuesPerEntry = bVectorMode ? 5 : 4;

	TArray<TSharedPtr<FMLAdapter::FSpace> > Spaces;
	Spaces.AddZeroed(TargetsToSenseCount);
	for (int Index = 0; Index < TargetsToSenseCount; ++Index)
	{
		// enemy heading, enemy distance, enemy ID
		Spaces[Index] = MakeShareable(new FMLAdapter::FSpace_Box({ ValuesPerEntry }));
	}
	Result = new FMLAdapter::FSpace_Tuple(Spaces);
	
	return MakeShareable(Result);
}

void UMLAdapterSensor_AIPerception::UpdateSpaceDef()	
{
	Super::UpdateSpaceDef();

	CachedTargets.Reset(TargetsToSenseCount);
	CachedTargets.AddDefaulted(TargetsToSenseCount);
}

void UMLAdapterSensor_AIPerception::OnAvatarSet(AActor* Avatar)
{
	Super::OnAvatarSet(Avatar);
	
	PerceptionComponent = nullptr;

	AController* Controller = nullptr;
	APawn* Pawn = nullptr;
	if (FMLAdapterAgentHelpers::GetAsPawnAndController(Avatar, Controller, Pawn))
	{
		UWorld* World = Avatar->GetWorld();
		// if at this point the World is null something is seriously wrong
		check(World);
		UMLAdapterManager::Get().EnsureAISystemPresence(*World);

		UAISystem* AISystem = UAISystem::GetCurrent(*World);
		if (ensure(AISystem) && ensure(AISystem->GetPerceptionSystem()))
		{
			UObject* Outer = Controller ? (UObject*)Controller : (UObject*)Pawn;
			PerceptionComponent = NewObject<UAIPerceptionComponent>(Outer);
			check(PerceptionComponent);

			UAISenseConfig_Sight* SightConfig = NewObject<UAISenseConfig_Sight>(this, UAISenseConfig_Sight::StaticClass(), TEXT("UAISenseConfig_Sight"));
			check(SightConfig);
			SightConfig->SightRadius = 50000;
			SightConfig->LoseSightRadius = 53000;
			SightConfig->PeripheralVisionAngleDegrees = PeripheralVisionAngleDegrees;
			SightConfig->AutoSuccessRangeFromLastSeenLocation = FAISystem::InvalidRange;
			SightConfig->SetMaxAge(MaxStimulusAge);
			PerceptionComponent->ConfigureSense(*SightConfig);
			PerceptionComponent->RegisterComponent();
		}
	}
}

void UMLAdapterSensor_AIPerception::GetViewPoint(AActor& Avatar, FVector& POVLocation, FRotator& POVRotation) const
{
	APlayerController* PC = Cast<APlayerController>(&Avatar);
	/*if (PC && PC->PlayerCameraManager && false)
	{
		PC->PlayerCameraManager->GetCameraViewPoint(POVLocation, POVRotation);
	}
	else*/
	{
		Avatar.GetActorEyesViewPoint(POVLocation, POVRotation);
	}
}

void UMLAdapterSensor_AIPerception::SenseImpl(const float DeltaTime)
{
	AActor* Avatar = GetAgent().GetAvatar();
	TArray<FTargetRecord> TmpCachedTargets;
	TmpCachedTargets.Reserve(TargetsToSenseCount);
	if (PerceptionComponent && Avatar)
	{
		TArray<AActor*> KnownActors;
		PerceptionComponent->GetKnownPerceivedActors(UAISense_Sight::StaticClass(), KnownActors);
		FVector POVLocation;
		FRotator POVRotation;
		GetViewPoint(*Avatar, POVLocation, POVRotation);

		for (AActor* Actor : KnownActors)
		{
			if (Actor)
			{
				FTargetRecord& TargetRecord = TmpCachedTargets.AddDefaulted_GetRef();

				const FVector ActorLocation = Actor->GetActorLocation();
				const FRotator ToTarget = (ActorLocation - POVLocation).ToOrientationRotator();
				
				TargetRecord.HeadingRotator = Sanify(ToTarget - POVRotation);
				TargetRecord.HeadingVector = TargetRecord.HeadingRotator.Vector();
				TargetRecord.Distance = FVector::Dist(POVLocation, ActorLocation);
				TargetRecord.ID = Actor->GetUniqueID();
				TargetRecord.HeadingDot = FVector::DotProduct(TargetRecord.HeadingVector, FVector::ForwardVector);
				TargetRecord.Target = Actor;
			}
		}
		
		if (TmpCachedTargets.Num() > 1)
		{
			TmpCachedTargets.SetNum(TargetsToSenseCount, EAllowShrinking::No);

			switch (TargetsSortType)
			{
			case ESortType::InFrontness:
				TmpCachedTargets.StableSort([](const FTargetRecord& A, const FTargetRecord& B) {
					return A.HeadingDot > B.HeadingDot || B.HeadingDot <= -1.f;
					});
				break;

			case ESortType::Distance:
			default:
				TmpCachedTargets.StableSort([](const FTargetRecord& A, const FTargetRecord& B) {
					// 0 means uninitialized so we send it of the back
					return A.Distance < B.Distance || B.Distance == 0.f; 
					});
				break;
			}
		}
	}

#if WITH_GAMEPLAY_DEBUGGER
	DebugRuntimeString = FString::Printf(TEXT("{white}%s"), (TmpCachedTargets.Num() > 0) ? *FString::Printf(TEXT("see %d"), TmpCachedTargets.Num()) : TEXT(""));
#endif // WITH_GAMEPLAY_DEBUGGER

	// fill up to TargetsToSenseCount with blanks
	for (int Index = TmpCachedTargets.Num(); Index < TargetsToSenseCount; ++Index)
	{
		TmpCachedTargets.Add(FTargetRecord());
	}

	FScopeLock Lock(&ObservationCS);
	CachedTargets = TmpCachedTargets;
}	

void UMLAdapterSensor_AIPerception::GetObservations(FMLAdapterMemoryWriter& Ar)
{
	FScopeLock Lock(&ObservationCS);

	FMLAdapter::FSpaceSerializeGuard SerializeGuard(SpaceDef, Ar);
	check(CachedTargets.Num() <= TargetsToSenseCount);

	for (int Index = 0; Index < TargetsToSenseCount; ++Index)
	{
		FTargetRecord& TargetData = CachedTargets[Index];
		Ar.Serialize(&TargetData.ID, sizeof(TargetData.ID));
		Ar.Serialize(&TargetData.Distance, sizeof(float));
		if (bVectorMode)
		{
			FVector3f HeadingVector3f = (FVector3f)TargetData.HeadingVector;
			Ar << HeadingVector3f;
		}
		else
		{
			float Pitch = TargetData.HeadingRotator.Pitch;
			float Yaw = TargetData.HeadingRotator.Yaw;
			Ar << Pitch << Yaw;
		}
	}
}

#if WITH_GAMEPLAY_DEBUGGER
#include "GameplayDebuggerCategory.h"
void UMLAdapterSensor_AIPerception::DescribeSelfToGameplayDebugger(FGameplayDebuggerCategory& DebuggerCategory) const
{
	int ValidTargets = 0;
	AActor* Avatar = GetAgent().GetAvatar();
	if (Avatar)
	{
		FVector POVLocation;
		FRotator POVRotation;
		GetViewPoint(*Avatar, POVLocation, POVRotation);

		for (int Index = 0; Index < TargetsToSenseCount; ++Index)
		{
			const FTargetRecord& TargetData = CachedTargets[Index];

			if (TargetData.ID == 0)
			{
				break;
			}

			++ValidTargets;
			DebuggerCategory.AddShape(FGameplayDebuggerShape::MakeSegment(POVLocation
				, POVLocation + (POVRotation + TargetData.HeadingRotator).Vector() * TargetData.Distance
				, FColor::Purple));
		}
	}

	Super::DescribeSelfToGameplayDebugger(DebuggerCategory);
}
#endif // WITH_GAMEPLAY_DEBUGGER
