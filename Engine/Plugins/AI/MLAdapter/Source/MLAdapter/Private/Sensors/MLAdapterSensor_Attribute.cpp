// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sensors/MLAdapterSensor_Attribute.h"
#include "AttributeSet.h"
#include "GameFramework/Actor.h"


UMLAdapterSensor_Attribute::UMLAdapterSensor_Attribute(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	TickPolicy = EMLAdapterTickPolicy::EveryTick;
}

bool UMLAdapterSensor_Attribute::ConfigureForAgent(UMLAdapterAgent& Agent)
{
	return false;
}

void UMLAdapterSensor_Attribute::Configure(const TMap<FName, FString>& Params)
{
	Super::Configure(Params);

	const FName NAME_Attributes = TEXT("attributes");
	const FString* AttributesValue = Params.Find(NAME_Attributes);
	if (AttributesValue != nullptr)
	{
		TArray<FString> Tokens;
		AttributesValue->ParseIntoArrayWS(Tokens, TEXT(","));
		SetAttributes(Tokens);
	}
}

void UMLAdapterSensor_Attribute::SenseImpl(const float DeltaTime)
{
	Values.Reset(AttributeNames.Num());
	if (AttributeSet == nullptr)
	{	
		Values.AddZeroed(AttributeNames.Num());
		return;
	}
	
	int Index = 0;
	for (const FGameplayAttributeData* Attribute : Attributes)
	{
		Values.Add(Attribute ? Attribute->GetCurrentValue() :0.f);
	}
}

void UMLAdapterSensor_Attribute::OnAvatarSet(AActor* Avatar)
{
	Super::OnAvatarSet(Avatar);

	if (Avatar)
	{
		BindAttributes(*Avatar);
	}
	else
	{
		AttributeSet = nullptr;
		Attributes.Reset(AttributeNames.Num());
	}
}

void UMLAdapterSensor_Attribute::GetObservations(FMLAdapterMemoryWriter& Ar)
{
	FScopeLock Lock(&ObservationCS);
	
	FMLAdapter::FSpaceSerializeGuard SerializeGuard(SpaceDef, Ar);
	Ar.Serialize(Values.GetData(), Values.Num() * sizeof(float));
}

TSharedPtr<FMLAdapter::FSpace> UMLAdapterSensor_Attribute::ConstructSpaceDef() const
{
	return MakeShareable(new FMLAdapter::FSpace_Box({ uint32(AttributeNames.Num()) }));
}

void UMLAdapterSensor_Attribute::UpdateSpaceDef()
{
	Super::UpdateSpaceDef();

	Values.Reset(AttributeNames.Num());
	Values.AddZeroed(AttributeNames.Num());
}

void UMLAdapterSensor_Attribute::SetAttributes(TArray<FString>& InAttributeNames)
{
	AttributeNames.Reset(InAttributeNames.Num());
	for (const FString& StringName : InAttributeNames)
	{
		AttributeNames.Add(FName(StringName));
	}
	
	AActor* Avatar = GetAvatar();
	if (Avatar)
	{
		BindAttributes(*Avatar);
	}

	UpdateSpaceDef();
}

void UMLAdapterSensor_Attribute::BindAttributes(AActor& Actor)
{
	TArray<UAttributeSet*> AttributeSetsFound;

	// 1. find UAttributeSet-typed property in Actor
	// 2. parse through found property instance looking for Attributes
	for (TFieldIterator<FObjectProperty> It(Actor.GetClass(), EFieldIteratorFlags::IncludeSuper); It; ++It)
	{
		const FObjectProperty* Prop = *It;
		if (Prop->PropertyClass->IsChildOf(UAttributeSet::StaticClass()))
		{
			AttributeSetsFound.Add(Cast<UAttributeSet>(Prop->GetObjectPropertyValue_InContainer(&Actor)));
		}
	}

	Attributes.Reset(AttributeNames.Num());
	if (AttributeSetsFound.Num() > 0)
	{
		ensureMsgf(AttributeSetsFound.Num() == 1, TEXT("Found %d attribute sets, using the first one"), AttributeSetsFound.Num());
		AttributeSet = AttributeSetsFound[0];
		UClass* AttributeSetClass = AttributeSet->GetClass();
		for (const FName& Name : AttributeNames)
		{
			FStructProperty* Prop = FindFProperty<FStructProperty>(AttributeSetClass, Name);
			if (Prop)
			{
				Attributes.Add(Prop->ContainerPtrToValuePtr<FGameplayAttributeData>(AttributeSet));
			}
			else
			{
				Attributes.Add(nullptr);
			}
		}
	}
	else
	{
		// not found
		// @todo log
		AttributeSet = nullptr;
	}
}
