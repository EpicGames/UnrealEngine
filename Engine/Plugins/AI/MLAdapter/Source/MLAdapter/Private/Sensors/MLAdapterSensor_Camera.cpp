// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sensors/MLAdapterSensor_Camera.h"
#include "Agents/MLAdapterAgent.h"
#include "Components/SceneCaptureComponent2D.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Controller.h"
#include "Camera/CameraComponent.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Debug/DebugHelpers.h"
#include "Camera/CameraActor.h"
#include "GameFramework/PlayerController.h"
#include "Engine/LocalPlayer.h"
#include "Engine/GameViewportClient.h"
#include "GameFramework/GameUserSettings.h"
#include "TextureResource.h"
#include "UnrealClient.h"
#include "UnrealEngine.h"

#include "Misc/ScopeLock.h"
#include <vector>


UMLAdapterSensor_Camera::UMLAdapterSensor_Camera(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Width = 160;
	Height = 160;

	TickPolicy = EMLAdapterTickPolicy::EveryXSeconds;
	TickEvery.Seconds = 0.3f;
}

bool UMLAdapterSensor_Camera::ConfigureForAgent(UMLAdapterAgent& Agent)
{
	if (Agent.GetAvatar())
	{
		OnAvatarSet(Agent.GetAvatar());
	}

	return Super::ConfigureForAgent(Agent);
}

void UMLAdapterSensor_Camera::Configure(const TMap<FName, FString>& Params)
{
	const FName NAME_Width = TEXT("width");
	const FName NAME_W = TEXT("w");
	const FName NAME_Height = TEXT("height");
	const FName NAME_H = TEXT("h");
	const FName NAME_CameraIndex = TEXT("camera_index");
	const FName NAME_CaptureSource = TEXT("capture_source");
	//const FName NAME_UI already defined

	Super::Configure(Params);

	for (auto KeyValue : Params)
	{
		if (KeyValue.Key == NAME_W || KeyValue.Key == NAME_Width)
		{
			ensure(KeyValue.Value.Len() > 0);
			Width = uint32(FCString::Atoi(*KeyValue.Value));
		}
		else if (KeyValue.Key == NAME_H || KeyValue.Key == NAME_Height)
		{
			ensure(KeyValue.Value.Len() > 0);
			Height = uint32(FCString::Atoi(*KeyValue.Value));
		}
		else if (KeyValue.Key == NAME_UI)
		{
			ensure(false && "Untested");
			bool bValue = bShowUI;
			LexFromString(bValue, *KeyValue.Value);
			bShowUI = bValue;
		}
	}

	UpdateSpaceDef();
}

TSharedPtr<FMLAdapter::FSpace> UMLAdapterSensor_Camera::ConstructSpaceDef() const
{
	return MakeShareable(new FMLAdapter::FSpace_Box({ Width, Height, 4 }, 0, 255));
}

void UMLAdapterSensor_Camera::HandleScreenshotData(int32 InSizeX, int32 InSizeY, const TArray<FColor>& InImageData)
{
	FScopeLock Lock(&ObservationCS);
	LastTickData.Reset(Width * Height);
}

void UMLAdapterSensor_Camera::SenseImpl(const float DeltaTime)
{
	static FReadSurfaceDataFlags ReadSurfaceDataFlags;
	// This is super important to disable this!
	// Instead of using this flag, we will set the gamma to the correct value directly
	ReadSurfaceDataFlags.SetLinearToGamma(false); 

	// fill current data cache with current screenshot
	if (CaptureComp == nullptr || RenderTarget2D == nullptr)
	{
		if (ScreenshotDataCallbackHandle.IsValid())
		{
			FScreenshotRequest::RequestScreenshot(bShowUI);
			ensure(FScreenshotRequest::IsScreenshotRequested());
		}
		return;
	}
	
	ensure(Width == uint32(RenderTarget2D->SizeX) && Height == uint32(RenderTarget2D->SizeY));

	FTextureRenderTargetResource* RenderTargetResource = RenderTarget2D->GameThread_GetRenderTargetResource();
	
	{
		FScopeLock Lock(&ObservationCS);
		LastTickData.Reset(Width * Height);
		RenderTargetResource->ReadLinearColorPixels(LastTickData, ReadSurfaceDataFlags);
	}
}

void UMLAdapterSensor_Camera::OnAvatarSet(AActor* Avatar)
{
	if (Avatar == nullptr)
	{
		if (CaptureComp)
		{
			CaptureComp->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
			CaptureComp->UnregisterComponent();
		}
		CaptureComp = nullptr;
		RenderTarget2D = nullptr;

		if (CachedViewportClient && ScreenshotDataCallbackHandle.IsValid())
		{
			CachedViewportClient->OnScreenshotCaptured().Remove(ScreenshotDataCallbackHandle);
		}
		CachedViewportClient = nullptr;

		ensure(false && "Remove old screenhot handle if needed");

		return;
	}

	AController* AsController = Cast<AController>(Avatar);
	Avatar = AsController && AsController->GetPawn() ? AsController->GetPawn().Get() : Avatar;

	TArray<UCameraComponent*> CameraComponents;
	Avatar->GetComponents(CameraComponents, /*bIncludeFromChildActors=*/false);

	for (UCameraComponent* CameraComp : CameraComponents)
	{
		if (CameraComp)
		{
			CaptureComp = NewObject<USceneCaptureComponent2D>(CameraComp, USceneCaptureComponent2D::StaticClass());
			if (CaptureComp)
			{
				CaptureComp->CaptureSource = CaptureSource;
				CaptureComp->AttachToComponent(CameraComp, FAttachmentTransformRules::KeepRelativeTransform);
				
				RenderTarget2D = NewObject<UTextureRenderTarget2D>(CameraComp);
				//check(NewRenderTarget2D);
				RenderTarget2D->RenderTargetFormat = RenderTargetFormat;
				RenderTarget2D->InitAutoFormat(Width, Height);
				RenderTarget2D->UpdateResourceImmediate(true);

				CaptureComp->TextureTarget = RenderTarget2D;
				CaptureComp->RegisterComponent();
			}
			break;
		}
	}

	APlayerController* AsPC = Cast<APlayerController>(AsController);
	if (AsPC)
	{
		ULocalPlayer* LocalPlayer = AsPC->GetLocalPlayer();
		if (LocalPlayer && LocalPlayer->ViewportClient)
		{
			ScreenshotDataCallbackHandle = LocalPlayer->ViewportClient->OnScreenshotCaptured().AddUObject(this, &UMLAdapterSensor_Camera::HandleScreenshotData);
			CachedViewportClient = LocalPlayer->ViewportClient;

			UGameUserSettings* UserSettings = GEngine->GetGameUserSettings();
			if (ensure(UserSettings))
			{
				const FIntPoint CurrentRes = UserSettings->GetScreenResolution();

				if (Width != CurrentRes.X || Height != CurrentRes.Y)
				{
					FSystemResolution::RequestResolutionChange(Width, Height, EWindowMode::Windowed);
				}
			}
		}
	}


	Super::OnAvatarSet(Avatar);
}

void UMLAdapterSensor_Camera::ClearPawn(APawn& InPawn)
{
	CaptureComp = nullptr;
}

void UMLAdapterSensor_Camera::GetObservations(FMLAdapterMemoryWriter& Ar)
{
	if (LastTickData.Num() == 0)
	{
		LastTickData.AddZeroed(Width*Height);
	}

	FScopeLock Lock(&ObservationCS);
	FMLAdapter::FSpaceSerializeGuard SerializeGuard(SpaceDef, Ar);	
	Ar.Serialize(LastTickData.GetData(), LastTickData.Num() * sizeof(FLinearColor));
	
	LastTickData.Reset();
}
