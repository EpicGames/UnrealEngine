// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sensors/MLAdapterSensor_EnhancedInput.h"
#include "Engine/LocalPlayer.h"
#include "EnhancedInputSubsystems.h"
#include "EnhancedInputComponent.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerInput.h"
#include "GameFramework/InputSettings.h"
#include "MLAdapterSpace.h"
#include "Engine/InputDelegateBinding.h"
#include "MLAdapterInputHelper.h"


void UMLAdapterSensor_EnhancedInput::Configure(const TMap<FName, FString>& Params)
{
	Super::Configure(Params);

	UpdateSpaceDef();
}

void UMLAdapterSensor_EnhancedInput::GetObservations(FMLAdapterMemoryWriter& Ar)
{
	FScopeLock Lock(&ObservationCS);

	FMLAdapter::FSpaceSerializeGuard SerializeGuard(SpaceDef, Ar);
	Ar.Serialize(InputState.GetData(), InputState.Num() * sizeof(float));

	InputState.Empty();
	InputState.SetNumZeroed(SpaceDef->Num());
}

TSharedPtr<FMLAdapter::FSpace> UMLAdapterSensor_EnhancedInput::ConstructSpaceDef() const
{
	return FMLAdapterInputHelper::ConstructEnhancedInputSpaceDef(TrackedActions);
}

void UMLAdapterSensor_EnhancedInput::UpdateSpaceDef()
{
	Super::UpdateSpaceDef();

	int32 ActionIndex = 0;
	for (const UInputAction* InputAction : TrackedActions)
	{
		InputStateIndices.Add(InputAction->GetName(), ActionIndex);

		switch (InputAction->ValueType)
		{
		case EInputActionValueType::Boolean:
		case EInputActionValueType::Axis1D:
			ActionIndex += 1;
			break;
		case EInputActionValueType::Axis2D:
			ActionIndex += 2;
			break;
		case EInputActionValueType::Axis3D:
			ActionIndex += 3;
			break;
		default:
			checkf(false, TEXT("Unsupported value type for input action value!"));
			break;
		}
	}

	InputState.SetNumZeroed(SpaceDef->Num());
}

void UMLAdapterSensor_EnhancedInput::OnAvatarSet(AActor* Avatar)
{
	if (Avatar == nullptr)
	{
		if (InputComponent != nullptr)
		{
			InputComponent->DestroyComponent();
		}

		return;
	}

	if (InputComponent != nullptr)
	{
		UE_LOGF(LogMLAdapter, Log, "MLAdapterSensor_EnhancedInput: This sensor already bound to another InputComponent. Cleaning up previous InputComponent and rebinding to new avatar.");
		InputComponent->DestroyComponent();
	}

	APawn* Pawn = Cast<APawn>(Avatar);
	if (Pawn == nullptr)
	{
		APlayerController* PC = Cast<APlayerController>(Avatar);
		if (PC != nullptr)
		{
			Pawn = PC->GetPawn();
		}
	}

	if (Pawn != nullptr)
	{
		UClass* PawnClass = Pawn->GetClass();
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (UInputDelegateBinding::SupportsInputDelegate(PawnClass))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			InputComponent = NewObject<UEnhancedInputComponent>(Pawn, UInputSettings::GetDefaultInputComponentClass());
			InputComponent->RegisterComponent();

			for (const UInputAction* InputAction : TrackedActions)
			{
				FEnhancedInputActionEventBinding& Binding = InputComponent->BindAction(InputAction, ETriggerEvent::Triggered, this, &UMLAdapterSensor_EnhancedInput::OnInputAction);
			}

			UInputDelegateBinding::BindInputDelegates(Pawn->GetClass(), InputComponent);
			UE_LOGF(LogMLAdapter, Log, "MLAdapterSensor_EnhancedInput: Successfully bound to %ls", *GetNameSafe(PawnClass));
		}
	}
	else
	{
		FString AvatarName = Avatar->GetName();
		UE_LOGF(LogMLAdapter, Warning, "MLAdapterSensor_EnhancedInput: Unable to bind - could not cast to APawn or APlayerController with a Pawn. Avatar name was %ls", *AvatarName);
	}

	Super::OnAvatarSet(Avatar);
}

void UMLAdapterSensor_EnhancedInput::OnInputAction(const FInputActionInstance& ActionInstance)
{
	const UInputAction* SourceAction = ActionInstance.GetSourceAction();

	int32* ActionIndexPtr = InputStateIndices.Find(SourceAction->GetName());
	if (ActionIndexPtr != nullptr)
	{
		FScopeLock Lock(&ObservationCS);

		int32 ActionIndex = *ActionIndexPtr;

		FInputActionValue ActionValue = ActionInstance.GetValue();
		switch (ActionValue.GetValueType())
		{
		case EInputActionValueType::Boolean:
		{
			bool Value = ActionValue.Get<bool>();
			InputState[ActionIndex] = Value ? 1.0f : 0.0f;
			break;
		}
		case EInputActionValueType::Axis1D:
		{
			float Value = ActionValue.Get<float>();
			InputState[ActionIndex] = Value;
			break;
		}
		case EInputActionValueType::Axis2D:
		{
			FVector2D Value = ActionValue.Get<FVector2D>();
			InputState[ActionIndex] = Value.X;
			InputState[ActionIndex + 1] = Value.Y;
			break;
		}
		case EInputActionValueType::Axis3D:
		{
			FVector Value = ActionValue.Get<FVector>();
			InputState[ActionIndex] = Value.X;
			InputState[ActionIndex + 1] = Value.Y;
			InputState[ActionIndex + 2] = Value.Z;
			break;
		}
		default:
			checkf(false, TEXT("Unsupported value type for input action value!"));
			break;
		}
	}
	else
	{
		FString ActionName = SourceAction->GetName();
		checkf(false, TEXT("OnInputAction received an ActionInstance from %s that we couldn't find in TrackedActions.  Was TrackedActions modified after binding?"), *ActionName);
	}
}
