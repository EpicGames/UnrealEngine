// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sensors/MLAdapterSensor_Input.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerInput.h"
#include "Engine/GameViewportClient.h"
#include "Engine/World.h"
#include "InputKeyEventArgs.h"
#include "MLAdapterInputHelper.h"
#include "MLAdapterSpace.h"
#include "Debug/DebugHelpers.h"


UMLAdapterSensor_Input::UMLAdapterSensor_Input(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bRecordKeyRelease = false;
}

void UMLAdapterSensor_Input::Configure(const TMap<FName, FString>& Params)
{
	Super::Configure(Params);

	const FName NAME_RecordRelease = TEXT("record_release");
	const FString* RecordReleaseValue = Params.Find(NAME_RecordRelease);
	if (RecordReleaseValue != nullptr)
	{
		bool bValue = bRecordKeyRelease;
		LexFromString(bValue, (TCHAR*)RecordReleaseValue);
		bRecordKeyRelease = bValue;
	}

	FMLAdapterInputHelper::CreateInputMap(InterfaceKeys, FKeyToInterfaceKeyMap);

	UpdateSpaceDef();
}

void UMLAdapterSensor_Input::GetObservations(FMLAdapterMemoryWriter& Ar)
{
	FScopeLock Lock(&ObservationCS);
	
	FMLAdapter::FSpaceSerializeGuard SerializeGuard(SpaceDef, Ar);
	Ar.Serialize(InputState.GetData(), InputState.Num() * sizeof(float));

	InputState.SetNumZeroed(SpaceDef->Num());
}

TSharedPtr<FMLAdapter::FSpace> UMLAdapterSensor_Input::ConstructSpaceDef() const 
{
	FMLAdapter::FSpace* Result = nullptr;

	const bool bHasButtons = (InterfaceKeys.Num() > 0);
	// mz@todo 
	const bool bHasAxis = false;// InterfaceAxis.Num();
	if (bHasButtons != bHasAxis)
	{
		if (bHasButtons)
		{
			Result = new FMLAdapter::FSpace_MultiDiscrete(InterfaceKeys.Num());
		}
		else // bHasAxis
		{
			NOT_IMPLEMENTED();
			Result = new FMLAdapter::FSpace_Dummy();
		}
	}
	else
	{
		Result = new FMLAdapter::FSpace_Tuple({
			MakeShareable(new FMLAdapter::FSpace_MultiDiscrete(InterfaceKeys.Num()))
			, MakeShareable(new FMLAdapter::FSpace_Box(/*InterfaceAxis.Num()*/{ 1 }))
			});
	}

	return MakeShareable(Result);
}

void UMLAdapterSensor_Input::UpdateSpaceDef()
{
	Super::UpdateSpaceDef();
	InputState.SetNumZeroed(SpaceDef->Num());
}

void UMLAdapterSensor_Input::OnAvatarSet(AActor* Avatar)
{
	if (Avatar == nullptr)
	{
		// clean up and exit
		if (GameViewport)
		{
			GameViewport->OnInputAxis().RemoveAll(this);
			GameViewport->OnInputKey().RemoveAll(this);
			GameViewport = nullptr;
		}

		return;
	}

	APlayerController* PC = Cast<APlayerController>(Avatar);
	if (PC == nullptr)
	{
		return;
	}

	UWorld* World = Avatar->GetWorld();
	if (World)
	{
		GameViewport = World->GetGameViewport();
		if (GameViewport)
		{
			GameViewport->OnInputAxis().AddUObject(this, &UMLAdapterSensor_Input::OnInputAxis);
			GameViewport->OnInputKey().AddUObject(this, &UMLAdapterSensor_Input::OnInputKey);
		}
	}

	Super::OnAvatarSet(Avatar);
}

void UMLAdapterSensor_Input::OnInputAxis(FViewport* InViewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime, int32 NumSamples, bool bGamepad)
{

}

void UMLAdapterSensor_Input::OnInputKey(const FInputKeyEventArgs& EventArgs)
{
	const FName KeyName = EventArgs.Key.GetFName();
	int32* InterfaceKey = FKeyToInterfaceKeyMap.Find(KeyName);
	if (InterfaceKey && ((EventArgs.Event != IE_Released)|| bRecordKeyRelease))
	{
		FScopeLock Lock(&ObservationCS);
		ensure(false && "save in FSpace format");
		InputState[*InterfaceKey] = 1;
	}
}
