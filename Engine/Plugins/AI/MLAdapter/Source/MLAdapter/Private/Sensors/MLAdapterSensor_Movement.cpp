// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sensors/MLAdapterSensor_Movement.h"
#include "GameFramework/Controller.h"
#include "GameFramework/Pawn.h"
#include "Agents/MLAdapterAgent.h"


UMLAdapterSensor_Movement::UMLAdapterSensor_Movement(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	TickPolicy = EMLAdapterTickPolicy::EveryTick;

	bAbsoluteLocation = true;
	bAbsoluteVelocity = true;
}

bool UMLAdapterSensor_Movement::ConfigureForAgent(UMLAdapterAgent& Agent)
{
	return false;
}

void UMLAdapterSensor_Movement::Configure(const TMap<FName, FString>& Params)
{
	Super::Configure(Params);

	const FName LocationKeyName = TEXT("location");
	const FString* LocationKeyValue = Params.Find(LocationKeyName);
	if (LocationKeyValue != nullptr)
	{
		bAbsoluteLocation = (LocationKeyValue->Find(TEXT("absolute")) != INDEX_NONE);
	}

	const FName VelocityKeyName = TEXT("velocity");
	const FString* VelocityKeyValue = Params.Find(VelocityKeyName);
	if (VelocityKeyValue != nullptr)
	{
		bAbsoluteVelocity = (VelocityKeyValue->Find(TEXT("absolute")) != INDEX_NONE);
	}

	UpdateSpaceDef();
}

void UMLAdapterSensor_Movement::SenseImpl(const float DeltaTime)
{
	AActor* Avatar = GetAgent().GetAvatar();

	if (Avatar == nullptr)
	{
		return;
	}
	
	AController* Controller = Cast<AController>(Avatar);
	CurrentLocation = (Controller && Controller->GetPawn()) ? Controller->GetPawn()->GetActorLocation() : Avatar->GetActorLocation();
	CurrentVelocity = (Controller && Controller->GetPawn()) ? Controller->GetPawn()->GetVelocity() : Avatar->GetVelocity();
}

void UMLAdapterSensor_Movement::OnAvatarSet(AActor* Avatar)
{
	Super::OnAvatarSet(Avatar);

	if (Avatar)
	{
		CurrentLocation = Avatar->GetActorLocation();
		CurrentVelocity = Avatar->GetVelocity();
	}
	else
	{
		CurrentLocation = CurrentVelocity = FVector::ZeroVector;
	}
}

void UMLAdapterSensor_Movement::GetObservations(FMLAdapterMemoryWriter& Ar)
{
	FScopeLock Lock(&ObservationCS);
	FVector3f Location = FVector3f(bAbsoluteLocation ? CurrentLocation : (CurrentLocation - RefLocation));
	FVector3f Velocity = FVector3f(bAbsoluteVelocity ? CurrentVelocity : (CurrentVelocity - RefVelocity));
	
	FMLAdapter::FSpaceSerializeGuard SerializeGuard(SpaceDef, Ar);	
	Ar << Location << Velocity;
	
	CurrentLocation = RefLocation;
	CurrentVelocity = RefVelocity;
}

TSharedPtr<FMLAdapter::FSpace> UMLAdapterSensor_Movement::ConstructSpaceDef() const
{
	// Location + Velocity
	return MakeShareable(new FMLAdapter::FSpace_Box({ 6 }));
}
