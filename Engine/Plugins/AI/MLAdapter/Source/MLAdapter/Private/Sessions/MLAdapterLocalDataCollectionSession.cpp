// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sessions/MLAdapterLocalDataCollectionSession.h"
#include "Agents/MLAdapterAgent.h"
#include "Engine/GameInstance.h"
#include "MLAdapterTypes.h"
#include "Misc/FileHelper.h"
#include "HAL/FileManager.h"

void UMLAdapterLocalDataCollectionSession::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	if (!FilePath.Path.IsEmpty())
	{
		OutputFilePath += FilePath.Path + FGenericPlatformMisc::GetDefaultPathSeparator();
	}

	if (bPrefixOutputFilenameWithTimestamp)
	{
		OutputFilePath += FDateTime::Now().ToString() + "_";
	}

	if (!FileName.IsEmpty())
	{
		OutputFilePath += FileName;
	}
	else
	{
		OutputFilePath += "mladapter.data";
	}

	UE_LOGF(LogMLAdapter, Log, "LocalDataCollectionSession: Writing data to %ls", *OutputFilePath);
}

void UMLAdapterLocalDataCollectionSession::OnPostWorldInit(UWorld& World)
{
	Super::OnPostWorldInit(World);

	UGameInstance* GameInstance = GetGameInstance();
	if (GameInstance)
	{
		// This will be cleaned up in UMLAdapterLocalDataCollectionSession::Close()
		GameInstance->GetOnPawnControllerChanged().AddDynamic(this, &UMLAdapterLocalDataCollectionSession::OnPawnControllerChanged);
	}
}

void UMLAdapterLocalDataCollectionSession::OnPawnControllerChanged(APawn* InPawn, AController* InController)
{
	for (UMLAdapterAgent* Agent : Agents)
	{
		if (Agent->GetController() == InController)
		{
			PlayerControlledAgent = Agent;
			break;
		}
	}
}

void UMLAdapterLocalDataCollectionSession::Tick(float DeltaTime)
{
	UMLAdapterAgent* Agent = PlayerControlledAgent.Get();

	if (Agent ==  nullptr)
	{
		UE_LOGF(LogMLAdapter, Log, "LocalDataCollectionSession: Player-controlled agent not found yet.");
		return;
	}

	Agent->Sense(DeltaTime);
	TArray<uint8> Buffer;
	FMLAdapterMemoryWriter Writer(Buffer);
	Agent->GetObservations(Writer);

	const bool bSuccess = FFileHelper::SaveArrayToFile(Buffer, *OutputFilePath, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
	if (!bSuccess)
	{
		UE_LOGF(LogMLAdapter, Error, "LocalDataCollectionSession: Failed to write data to output file.");
	}
}

void UMLAdapterLocalDataCollectionSession::Close()
{
	Super::Close();

	if (CachedWorld && CachedWorld->GetGameInstance())
	{
		CachedWorld->GetGameInstance()->GetOnPawnControllerChanged().RemoveAll(this);
	}
}
