// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sessions/MLAdapterSession.h"
#include "MLAdapterTypes.h"
#include "MLAdapterSettings.h"
#include "EngineUtils.h"
#include "Engine/GameInstance.h"
#include "GameFramework/GameMode.h"
#include "GameFramework/GameStateBase.h"
#include "Agents/MLAdapterAgent.h"
#include "Managers/MLAdapterManager.h"


void UMLAdapterSession::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		bTickWorldManually = (UMLAdapterManager::Get().IsWorldRealTime() == false);
	}
}

void UMLAdapterSession::BeginDestroy()
{
	Super::BeginDestroy();
}

void UMLAdapterSession::SetManualWorldTickEnabled(bool bEnable)
{
	bTickWorldManually = bEnable;
	if (bEnable)
	{
		if (WorldTicker == nullptr && CachedWorld)
		{
			WorldTicker = MakeShareable(new FWorldTicker(CachedWorld));
		}
	}
	else
	{
		WorldTicker = nullptr;
	}
}

void UMLAdapterSession::EnableActionDuration(FMLAdapter::FAgentID AgentID, bool bEnable, float DurationSeconds)
{
	UMLAdapterAgent* Agent = GetAgent(AgentID);
	if (Agent != nullptr)
	{
		Agent->EnableActionDuration(bEnable, DurationSeconds);		
	}
}

bool UMLAdapterSession::TryResetActionDuration(FMLAdapter::FAgentID AgentID)
{
	UMLAdapterAgent* Agent = GetAgent(AgentID);
	if (Agent != nullptr)
	{
		return Agent->TryResetActionDuration();
	}

	return false;
}

void UMLAdapterSession::FWorldTicker::Tick(float DeltaTime) 
{
#if WITH_EDITORONLY_DATA
	if (CachedWorld.IsValid())
	{
		// will get cleared in Session::tick
		GIntraFrameDebuggingGameThread = true;
	}
#endif // WITH_EDITORONLY_DATA
}

UMLAdapterSession::FWorldTicker::~FWorldTicker()
{
#if WITH_EDITORONLY_DATA
	GIntraFrameDebuggingGameThread = false;
#endif // WITH_EDITORONLY_DATA
}

void UMLAdapterSession::SetWorld(UWorld* NewWorld)
{
	if (CachedWorld == NewWorld)
	{
		return;
	}

	WorldTicker = nullptr;

	if (CachedWorld)
	{
		RemoveAvatars(CachedWorld);

		SetGameMode(nullptr);
		CachedWorld->RemoveOnActorSpawnedHandler(ActorSpawnedDelegateHandle);
		ActorSpawnedDelegateHandle.Reset();
		CachedWorld = nullptr;
		
		LastTimestamp = -1.f;
	}

	if (NewWorld)
	{
		CachedWorld = NewWorld;
		LastTimestamp = NewWorld->GetTimeSeconds();
		SetGameMode(NewWorld->GetAuthGameMode());

		if (bTickWorldManually)
		{
			WorldTicker = MakeShareable(new FWorldTicker(NewWorld));
		}

		FindAvatars(*CachedWorld);
		ActorSpawnedDelegateHandle = CachedWorld->AddOnActorSpawnedHandler(FOnActorSpawned::FDelegate::CreateUObject(this, &UMLAdapterSession::OnActorSpawned));
	}
}

void UMLAdapterSession::OnActorSpawned(AActor* InActor)
{
	// check if it's something we need to consider, like a pawn or controller

	// CONCERN: this function is going to be called a lot and we care only about
	// a couple of possible actor classes. Sometimes we don't event care at all. 
	// this should be bound on demand, only when there are unassigned agents @todo

	if (AwaitingAvatar.Num() == 0 || InActor == nullptr)
	{
		return;
	}

	ensure(AvatarToAgent.Find(HashAvatar(*InActor)) == nullptr);

	// @todo extremely wasteful! needs rethinking/reimplementation
	TArray<UMLAdapterAgent*> AwaitingAvatarCopy = AwaitingAvatar;
	AwaitingAvatar.Reset();

	bool bAssigned = false;
	for (UMLAdapterAgent* Agent : AwaitingAvatarCopy)
	{
		if (!bAssigned && Agent->IsSuitableAvatar(*InActor))
		{
			BindAvatar(*Agent, *InActor);
			if (Agent->GetAvatar() == InActor)
			{
				bAssigned = true;
			}
			else
			{
				AwaitingAvatar.Add(Agent);
			}
		}
		else
		{
			AwaitingAvatar.Add(Agent);
		}
	}		
}

void UMLAdapterSession::OnPostWorldInit(UWorld& World)
{
	SetWorld(&World);
}

void UMLAdapterSession::OnWorldCleanup(UWorld& World, bool bSessionEnded, bool bCleanupResources)
{
	if (CachedWorld == &World)
	{
		SetWorld(nullptr);
	}
}

void UMLAdapterSession::OnGameModeInitialized(AGameModeBase& GameModeBase)
{
	SetGameMode(&GameModeBase);
}

void UMLAdapterSession::SetGameMode(AGameModeBase* GameModeBase)
{
	CachedGameMode = GameModeBase;
	AGameMode* AsGameMode = Cast<AGameMode>(GameModeBase);
	if (AsGameMode)
	{
		OnGameModeMatchStateSet(AsGameMode->GetMatchState());
	}
	else
	{
		// a game not utilizing AGameMode's functionality is either a simple game
		// or a very sophisticated one. In the former case we just assume it's 
		// 'ready' from the very start:
		SimulationState = GameModeBase ? EMLAdapterSimState::InProgress : EMLAdapterSimState::Finished;
		// in the latter case we put it on the user to override this logic.
	}

	// game-specific data extraction will come here
}

void UMLAdapterSession::OnGameModePostLogin(AGameModeBase* GameMode, APlayerController* NewPlayer)
{
	if (ActorSpawnedDelegateHandle.IsValid() == false && CachedWorld)
	{
		FindAvatars(*CachedWorld);
		ActorSpawnedDelegateHandle = CachedWorld->AddOnActorSpawnedHandler(FOnActorSpawned::FDelegate::CreateUObject(this, &UMLAdapterSession::OnActorSpawned));
	}
}

void UMLAdapterSession::OnGameModeMatchStateSet(FName InMatchState)
{
	if (InMatchState == MatchState::EnteringMap)
	{
		SimulationState = EMLAdapterSimState::BootingUp;
	}
	else if (InMatchState == MatchState::WaitingToStart)
	{
		SimulationState = EMLAdapterSimState::BootingUp;

		// no point in binding sooner than this 
		if (ensure(CachedWorld))
		{
			FindAvatars(*CachedWorld);
			if (ActorSpawnedDelegateHandle.IsValid() == false)
			{
				ActorSpawnedDelegateHandle = CachedWorld->AddOnActorSpawnedHandler(FOnActorSpawned::FDelegate::CreateUObject(this, &UMLAdapterSession::OnActorSpawned));
			}
		}
	}
	else if (InMatchState == MatchState::InProgress)
	{
		SimulationState = EMLAdapterSimState::InProgress;
	}
	else if (InMatchState == MatchState::WaitingPostMatch
		|| InMatchState == MatchState::LeavingMap
		|| InMatchState == MatchState::Aborted)
	{
		SimulationState = EMLAdapterSimState::Finished;
	}
}

void UMLAdapterSession::Open()
{
	bActive = true;
}

void UMLAdapterSession::Close()
{
	bActive = false;

	// 'destroy' agents by clearing the async flag and letting the GC clean them
	for (UMLAdapterAgent* Agent : Agents)
	{
		Agent->ClearInternalFlags(EInternalObjectFlags::Async);
	}

	Agents.Reset();
}

void UMLAdapterSession::Tick(float DeltaTime)
{
	LastTimestamp = CachedWorld ? CachedWorld->GetTimeSeconds() : -1.f;
	
	// @todo for perf reasons we could grab all the agents' senses and tick them
	// by class to keep the cache hot
	for (UMLAdapterAgent* Agent : Agents)
	{
		Agent->Sense(DeltaTime);
	}

	for (UMLAdapterAgent* Agent : Agents)
	{
		Agent->Think(DeltaTime);
	}

	for (UMLAdapterAgent* Agent : Agents)
	{
		Agent->Act(DeltaTime);
	}

#if WITH_EDITORONLY_DATA
	if (CachedWorld && GIntraFrameDebuggingGameThread)
	{
		GIntraFrameDebuggingGameThread = false;
		// the WorldTicker will clear it to allow next tick
	}
#endif // WITH_EDITORONLY_DATA
}

void UMLAdapterSession::ConfigureAsServer()
{

}

void UMLAdapterSession::ConfigureAsClient()
{

}

void UMLAdapterSession::ResetWorld(FMLAdapter::FAgentID AgentID)
{
	if (CachedGameMode == nullptr)
	{
		return;
	}

	if (AgentID != FMLAdapter::InvalidAgentID)
	{
		if (Agents.IsValidIndex(AgentID) && GetAgent(AgentID)->GetAvatar())
		{
			AController* AvatarController = FMLAdapter::ActorToController(*GetAgent(AgentID)->GetAvatar());
			CachedGameMode->RestartPlayer(AvatarController);
		}
	}
	else
	{
		for (UMLAdapterAgent* Agent : Agents)
		{
			if (Agent->GetAvatar())
			{
				AController* AvatarController = FMLAdapter::ActorToController(*Agent->GetAvatar());
				CachedGameMode->RestartPlayer(AvatarController);
			}
		}
	}
}

bool UMLAdapterSession::IsDone() const
{ 
	return (SimulationState == EMLAdapterSimState::Finished)
		|| (CachedGameMode != nullptr
			&& CachedGameMode->HasMatchEnded());
}

bool UMLAdapterSession::IsReady() const
{
	return SimulationState == EMLAdapterSimState::InProgress 
		&& (CachedGameMode != nullptr)
		&& (CachedGameMode->HasMatchStarted() == true)
		&& (CachedGameMode->HasMatchEnded() == false);
}

// todo bmulcahy we probably need to support AddAgent(UClass* Agent) for games that have multiple agents of various types
FMLAdapter::FAgentID UMLAdapterSession::AddAgent()
{
	FScopeLock Lock(&AgentOpCS);

	UClass* AgentClass = UMLAdapterSettings::GetAgentClass().Get()
		? UMLAdapterSettings::GetAgentClass().Get()
		: UMLAdapterAgent::StaticClass();

	UE_LOGF(LogMLAdapter, Log, "Creating MLAdapter agent of class %ls", *GetNameSafe(AgentClass));

	UMLAdapterAgent* NewAgent = FMLAdapter::NewObject<UMLAdapterAgent>(this, AgentClass);

	NewAgent->SetAgentID(Agents.Add(NewAgent));

	return NewAgent->GetAgentID();
}

FMLAdapter::FAgentID UMLAdapterSession::AddAgent(const FMLAdapterAgentConfig& InConfig)
{
	FScopeLock Lock(&AgentOpCS);

	UClass* AgentClass = FMLAdapterLibrarian::Get().FindAgentClass(InConfig.AgentClassName);
	UMLAdapterAgent* NewAgent = FMLAdapter::NewObject<UMLAdapterAgent>(this, AgentClass);
	NewAgent->SetAgentID(Agents.Add(NewAgent));

	NewAgent->Configure(InConfig);

	return NewAgent->GetAgentID();
}

FMLAdapter::FAgentID UMLAdapterSession::GetNextAgentID(FMLAdapter::FAgentID ReferenceAgentID) const
{
	if (Agents.Num() == 0)
	{
		return FMLAdapter::InvalidAgentID;
	}
		
	int Index = (ReferenceAgentID != FMLAdapter::InvalidAgentID)
		? (int(ReferenceAgentID) + 1) % Agents.Num()
		: 0;

	for (int Iter = 0; Iter < Agents.Num(); ++Iter)
	{
		if (Agents[Index])
		{
			return FMLAdapter::FAgentID(Index);
		}
		Index = (Index + 1) % Agents.Num();
	}

	return FMLAdapter::InvalidAgentID;
}

UMLAdapterAgent* UMLAdapterSession::GetAgent(FMLAdapter::FAgentID AgentID)
{
	if (Agents.IsValidIndex(AgentID) == false)
	{
		UE_LOGF(LogMLAdapter, Warning, "UMLAdapterSession::GetAgent: Invalid AgentID. Failing");
		return nullptr;
	}

	return Agents[AgentID];
}

void UMLAdapterSession::RemoveAgent(FMLAdapter::FAgentID AgentID)
{
	if (Agents.IsValidIndex(AgentID) == false)
	{
		return;
	}

	// remove from Agents only if it's the last agent in the list since 
	// external code refers to agents by ID which is an index to this array
	// @todo consider switching over to a TMap
	// @todo consider always nulling out, i.e. not removing elements from Agents
	UMLAdapterAgent* Agent = Agents[AgentID];
	if (!ensure(Agent))
	{
		return;
	}
	
	OnBeginAgentRemove.Broadcast(*Agent);

	if (Agents.Num() == AgentID)
	{
		Agents.Pop(EAllowShrinking::No);
	}
	else
	{
		Agents[AgentID] = nullptr;
	}
	for (auto It = AvatarToAgent.CreateIterator(); It; ++It)
	{
		if (It.Value() == Agent)
		{
			It.RemoveCurrent();
		}
	}
	AwaitingAvatar.RemoveSingleSwap(Agent, EAllowShrinking::No);
	// there should have been only one agent in AwaitingAvatar
	ensureMsgf(AwaitingAvatar.Find(Agent) == false, TEXT("there should have been only one agent in AwaitingAvatar"));

	Agent->ClearInternalFlags(EInternalObjectFlags::Async);
}

bool UMLAdapterSession::IsAgentReady(FMLAdapter::FAgentID AgentID) const
{
	return Agents.IsValidIndex(AgentID) && Agents[AgentID] && Agents[AgentID]->IsReady();
}

void UMLAdapterSession::FindAvatars(UWorld& World)
{
	// @todo naive implementation for now, subject to optimization in the future
	TArray<UMLAdapterAgent*> AwaitingAvatarCopy = AwaitingAvatar;
	AwaitingAvatar.Reset();

	for (UMLAdapterAgent* Agent : AwaitingAvatarCopy)
	{
		if (ensure(Agent))
		{
			ensure(Agent->GetAvatar() == nullptr); // if not then the avatar has been assigned outside of normal procedure 
			RequestAvatarForAgent(*Agent, &World);
		}
	}
}

void UMLAdapterSession::RemoveAvatars(UWorld* World)
{
	for (UMLAdapterAgent* Agent : Agents)
	{
		if (Agent && Agent->GetAvatar() && (World == nullptr || Agent->GetAvatar()->GetWorld() == World))
		{
			AActor* OldAvatar = Agent->GetAvatar();
			Agent->SetAvatar(nullptr);
			OnAgentAvatarChanged.Broadcast(*Agent, OldAvatar);
		}
	}
}

bool UMLAdapterSession::RequestAvatarForAgent(FMLAdapter::FAgentID& AgentID, UWorld* InWorld)
{
	UMLAdapterAgent* Agent = GetAgent(AgentID);
	return (Agent != nullptr) && RequestAvatarForAgent(*Agent, InWorld);
}

bool UMLAdapterSession::RequestAvatarForAgent(UMLAdapterAgent& Agent, UWorld* InWorld, const bool bForceSearch)
{
	if (Agent.GetAvatar() != nullptr)
	{
		// skipping.
		UE_LOGF(LogMLAdapter, Verbose, "UMLAdapterSession::RequestAvatarForAgent called for agent [%d] while it still has an avatar [%ls]. Call ClearAvatar first to null-out agent\'s avatar.",
			Agent.GetAgentID(), *GetNameSafe(Agent.GetAvatar()));
		return false;
	}
	if (bForceSearch == false && AwaitingAvatar.Find(&Agent) != INDEX_NONE)
	{
		// already waiting, skip
		return false;
	}

	// we're adding to awaiting list first to avoid calling RequestAvatarForAgent 
	// couple of times in sequence
	AwaitingAvatar.AddUnique(&Agent);

	InWorld = InWorld ? InWorld : ToRawPtr(CachedWorld);
	
	if (InWorld == nullptr)
	{
		UE_LOGF(LogMLAdapter, Warning, "UMLAdapterSession::RequestAvatarForAgent called with InWorld and CachedWorld both being null. Auto-failure.");
		return false;
	}

	AActor* Avatar = nullptr;
	if (InWorld != nullptr)
	{
		// @todo might want to make special cases for Controllers and 
		UClass* AvatarClass = Agent.AvatarClass;
		if (ensure(AvatarClass))
		{
			for (TActorIterator<AActor> It(InWorld, AvatarClass); It; ++It)
			{
				if (Agent.IsSuitableAvatar(**It)
					// AND not already an avatar
					&& AvatarToAgent.Find(HashAvatar(**It)) == nullptr)
				{
					Avatar = *It;
					break;
				}
			}
		}
	}

	if (Avatar)
	{
		BindAvatar(Agent, *Avatar);
		ensureMsgf(Agent.GetAvatar() == Avatar, TEXT("If we get here and the avatar setting fails it means the above process, leading to avatar selection was flawed"));
	}

	return Agent.GetAvatar() && (Agent.GetAvatar() == Avatar);
}

void UMLAdapterSession::BindAvatar(UMLAdapterAgent& Agent, AActor& Avatar)
{
	AActor* OldAvatar = Agent.GetAvatar();
	ClearAvatar(Agent);

	Agent.SetAvatar(&Avatar);
	AwaitingAvatar.RemoveSingleSwap(&Agent, EAllowShrinking::No);
	AvatarToAgent.Add(HashAvatar(Avatar), &Agent);

	OnAgentAvatarChanged.Broadcast(Agent, OldAvatar);
}

void UMLAdapterSession::ClearAvatar(UMLAdapterAgent& Agent)
{
	AActor* OldAvatar = Agent.GetAvatar();
	if (OldAvatar == nullptr)
	{
		// it's possible the previous avatar is already gone. We need to look
		// through the map values
		const uint32* Key = AvatarToAgent.FindKey(&Agent);
		if (Key)
		{
			AvatarToAgent.Remove(*Key);
		}
		return;
	}

	decltype(AvatarToAgent)::ValueType BoundAgent = nullptr;
	AvatarToAgent.RemoveAndCopyValue(HashAvatar(*OldAvatar), BoundAgent);
	ensure(BoundAgent == &Agent);
	// @todo what if it causes another RequestAvatarForAgent call
	// should probably ignore the second one
	Agent.SetAvatar(nullptr);
	OnAgentAvatarChanged.Broadcast(Agent, OldAvatar);
}

const UMLAdapterAgent* UMLAdapterSession::FindAgentByAvatar(AActor& Avatar) const
{
	for (UMLAdapterAgent* Agent : Agents)
	{
		if (Agent && Agent->GetAvatar() == &Avatar)
		{
			return Agent;
		}
	}

	return nullptr;
}

#if WITH_GAMEPLAY_DEBUGGER
#include "GameplayDebuggerCategory.h"

void UMLAdapterSession::DescribeSelfToGameplayDebugger(FGameplayDebuggerCategory& DebuggerCategory) const
{
	if (Agents.Num() > 0)
	{
		bool bInvalidAgents = false;
		FString ActiveAgentIDs;
		for (UMLAdapterAgent* Agent : Agents)
		{
			if (Agent)
			{
				ActiveAgentIDs += FString::Printf(TEXT("{%s}%d,")
					, AwaitingAvatar.Find(Agent) != INDEX_NONE ? TEXT("grey") : TEXT("white")
					, Agent->GetAgentID());
			}
			else
			{
				bInvalidAgents = true;
			}
		}
		
		DebuggerCategory.AddTextLine(FString::Printf(TEXT("{green}Active agents: %s"), *ActiveAgentIDs));

		if (bInvalidAgents || AwaitingAvatar.Find(nullptr) != INDEX_NONE)
		{
			DebuggerCategory.AddTextLine(FString::Printf(TEXT("{red} invalid agents found!")));
		}
	}
}
#endif // WITH_GAMEPLAY_DEBUGGER
