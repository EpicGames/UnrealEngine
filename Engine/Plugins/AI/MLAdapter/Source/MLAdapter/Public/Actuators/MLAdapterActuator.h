// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "UObject/UObjectGlobals.h"
#include "HAL/CriticalSection.h"
#include "Agents/MLAdapterAgentElement.h"
#include "MLAdapterTypes.h"
#include "MLAdapterActuator.generated.h"


class AActor;
struct FMLAdapterDescription;

/** Allows an agent to take an action in the game world. */
UCLASS(Abstract, Blueprintable, EditInlineNew)
class MLADAPTER_API UMLAdapterActuator : public UMLAdapterAgentElement
{
	GENERATED_BODY()

public:
	UMLAdapterActuator(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual void PostInitProperties() override;

	/** Takes an action in the game based on earlier digested data. */
	virtual void Act(const float DeltaTime) {}

	/** Stores data from the ValueStream to be later used by Act. */
	virtual void DigestInputData(FMLAdapterMemoryReader& ValueStream) {}

protected:

	/**
	 * This needs to be used in every implementation of Act and DigestInputData to ensure that shared state is not be
	 * accessed simultaneously by the game thread and the RPC server thread.
	 */
	mutable FCriticalSection ActionCS;
};