// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "Actuators/MLAdapterActuator.h"
#include "MLAdapterActuator_Camera.generated.h"


/** Allows an agent to rotate the camera. */
UCLASS(Blueprintable, EditInlineNew)
class MLADAPTER_API UMLAdapterActuator_Camera : public UMLAdapterActuator
{
	GENERATED_BODY()

public:
	UMLAdapterActuator_Camera(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual void Configure(const TMap<FName, FString>& Params) override;

	virtual TSharedPtr<FMLAdapter::FSpace> ConstructSpaceDef() const override;

	/** Rotates the camera. */
	virtual void Act(const float DeltaTime) override;

	/** Stores data in HeadingVector or HeadingRotator depending on bVectorMode. */
	virtual void DigestInputData(FMLAdapterMemoryReader& ValueStream) override;

protected:

#if WITH_GAMEPLAY_DEBUGGER
	virtual void DescribeSelfToGameplayDebugger(class FGameplayDebuggerCategory& DebuggerCategory) const override;
#endif // WITH_GAMEPLAY_DEBUGGER

	/** Stores the direction to point the camera as a rotator. */
	UPROPERTY()
	FRotator HeadingRotator;

	/** Stores the direction to point the camera as a vector. */
	UPROPERTY()
	FVector HeadingVector;

	/** If true, acting will reinitialize the HeadingRotator/Vector after reading them. */
	UPROPERTY()
	uint32 bConsumeData : 1;

	/** If true, use the HeadingVector. Otherwise, use the HeadingRotator. */
	UPROPERTY()
	uint32 bVectorMode : 1;
};