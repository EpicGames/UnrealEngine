// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "Actuators/MLAdapterActuator.h"
#include "InputAction.h"
#include "MLAdapterActuator_EnhancedInput.generated.h"


/** Allows an agent to simulate player input via the UEnhancedPlayerInput subsystem. */
UCLASS(Blueprintable, EditInlineNew)
class MLADAPTER_API UMLAdapterActuator_EnhancedInput : public UMLAdapterActuator
{
	GENERATED_BODY()

public:
	virtual void Configure(const TMap<FName, FString>& Params) override;

	virtual TSharedPtr<FMLAdapter::FSpace> ConstructSpaceDef() const override;

	/** Performs one or more actions on the input component. */
	virtual void Act(const float DeltaTime) override;

	/** Stores data in InputData to be later injected into actions. */
	virtual void DigestInputData(FMLAdapterMemoryReader& ValueStream) override;

protected:
	TArray<float> InputData;

	/**
	* The actions that this actuator is allowed to inject data into. Modifying this list after using the actuator
	* is unsupported because the InputData is injected into the actions based on the ordering in this array. Changing
	* this could cause data to be injected incorrectly.
	*/
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = MLAdapter)
	TArray<TObjectPtr<UInputAction>> TrackedActions;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = MLAdapter)
	bool bClearActionOnUse = true;
};