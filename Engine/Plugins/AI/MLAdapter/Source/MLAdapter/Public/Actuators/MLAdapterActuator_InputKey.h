// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "Actuators/MLAdapterActuator.h"
#include "MLAdapterActuator_InputKey.generated.h"

/** Allows an agent to directly inject key presses into its avatar's input component. */
UCLASS(Blueprintable, EditInlineNew)
class MLADAPTER_API UMLAdapterActuator_InputKey : public UMLAdapterActuator
{
	GENERATED_BODY()

public:
	UMLAdapterActuator_InputKey(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual void Configure(const TMap<FName, FString>& Params) override;

	virtual TSharedPtr<FMLAdapter::FSpace> ConstructSpaceDef() const override;

	/** Presses the keys stored in "KeysToPress" */
	virtual void Act(const float DeltaTime) override;

	virtual void DigestInputData(FMLAdapterMemoryReader& ValueStream) override;

protected:
	TArray<TTuple<FKey, FName>> RegisteredKeys;

	TArray<int32> KeysToPress;

	TBitArray<> PressedKeys;

	TArray<float> InputData;

	/** temporary solution. If true then the incoming actions are expected to be 
	 *	MultiBinary, if false (default) the actions will be treated as Discrete */
	bool bIsMultiBinary;
};