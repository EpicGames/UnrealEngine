// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "MLAdapterTypes.h"
#include "Actuators/MLAdapterActuator.h"
#include "MLAdapterAgent.generated.h"


class UMLAdapterAction;
class UMLAdapterSensor;
class AController;
class APawn;
class UMLAdapterSession;
class UMLAdapterActuator;
struct FMLAdapterSpaceDescription;


/** Provides a serializable mapping from parameter name to value that is used to configure sensors & actuators. */
USTRUCT()
struct MLADAPTER_API FMLAdapterParameterMap
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = MLAdapter)
	TMap<FName, FString> Params;
};

/** A serializable config for an agent. Used in the external API to define agents. */
USTRUCT()
struct MLADAPTER_API FMLAdapterAgentConfig
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FName, FMLAdapterParameterMap> Sensors;

	UPROPERTY()
	TMap<FName, FMLAdapterParameterMap> Actuators;

	UPROPERTY()
	FName AvatarClassName;

	UPROPERTY()
	FName AgentClassName;

	/** If set to true, won't accept child classes of the AvatarClass. */
	UPROPERTY()
	bool bAvatarClassExact = false;

	UPROPERTY()
	bool bAutoRequestNewAvatarUponClearingPrev = true;

	FMLAdapterParameterMap& AddSensor(const FName SensorName, FMLAdapterParameterMap&& Parameters = FMLAdapterParameterMap());
	FMLAdapterParameterMap& AddActuator(const FName ActuatorName, FMLAdapterParameterMap&& Parameters = FMLAdapterParameterMap());
};

namespace FMLAdapterAgentHelpers
{
	/** Get the Pawn and Controller from the given Avatar if possible. */
	bool MLADAPTER_API GetAsPawnAndController(AActor* Avatar, AController*& OutController, APawn*& OutPawn);

	/** Get the Pawn and Controller from the given Avatar if possible. */
	bool MLADAPTER_API GetAsPawnAndController(AActor* Avatar, TObjectPtr<AController>& OutController, TObjectPtr<APawn>& OutPawn);
}

/**
 * An agent capable of controlling a single avatar (e.g. a Pawn or Controller). Contains sensors for
 * perceiving information about the environment and actuators for taking actions in the game.
 */
UCLASS(Blueprintable, EditInlineNew)
class MLADAPTER_API UMLAdapterAgent : public UObject
{
    GENERATED_BODY()

public:
	UMLAdapterAgent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Perform initial setup for blueprint spawned agents. */
	virtual void PostInitProperties() override;

	/** Shutdown all the sensors and actuators and cleanup references to the avatar. */
	virtual void BeginDestroy() override;

	/** Add a sensor to this agent. Returns true if the sensor was successfully added. */
	virtual bool RegisterSensor(UMLAdapterSensor& Sensor);
	
	/** Updates all the sensors that are configured as 'IsPolling'. */
	virtual void Sense(const float DeltaTime);
	
	/** 
	 * Decide what action to take based on the current observations.
	 * @see UMLAdapterAgent_Inference for an agent that implements this.
	 */
	virtual void Think(const float DeltaTime);

	/** Tick all of the agent's actuators. */
	virtual void Act(const float DeltaTime);
	
	/** Move data into the actuators for the next time Act is called. */
	virtual void DigestActions(FMLAdapterMemoryReader& ValueStream);

	/** Get this agent's ID. */
	FMLAdapter::FAgentID GetAgentID() const { return AgentID; }

	/** Get the Pawn this agent is controlling. */
	APawn* GetPawn() { return Pawn; }

	/** Get the Pawn this agent is controlling. */
	const APawn* GetPawn() const { return Pawn; }

	/** Get the Controller this agent is controlling. */
	AController* GetController() { return Controller; }

	/** Get the Controller this agent is controlling. */
	const AController* GetController() const { return Controller; }
	
	TArray<TObjectPtr<UMLAdapterSensor>>::TConstIterator GetSensorsConstIterator() const { return Sensors.CreateConstIterator(); }
	TArray<TObjectPtr<UMLAdapterActuator>>::TConstIterator GetActuatorsConstIterator() const { return Actuators.CreateConstIterator(); }
	
	/** If the avatar is a controller, then get the current score from the controller's player state. */
	virtual float GetReward() const;

	/** The agent is done if its avatar has been destroyed and it can't request a new one. */
	virtual bool IsDone() const;

	/** Get the actuator with the given ID if this agent has it. */
	UMLAdapterActuator* GetActuator(const uint32 ActuatorID) 
	{ 
		TObjectPtr<UMLAdapterActuator>* FoundActuator = Actuators.FindByPredicate([ActuatorID](const UMLAdapterActuator* Actuator) { return (Actuator->GetElementID() == ActuatorID); });
		return FoundActuator ? *FoundActuator : nullptr;
	}

#if WITH_GAMEPLAY_DEBUGGER
	void DescribeSelfToGameplayDebugger(class FGameplayDebuggerCategory& DebuggerCategory) const;
#endif // WITH_GAMEPLAY_DEBUGGER

protected:

	/** When the agent's avatar is destroyed, we need to cleanup callbacks and references to the avatar. Will request a new avatar if AgentConfig.bAutoRequestNewAvatarUponClearingPrev is true. */
	UFUNCTION()
	virtual void OnAvatarDestroyed(AActor* DestroyedActor);
	
	/** Will be bound to UGameInstance.OnPawnControllerChanged if current avatar is a pawn or a controller. */
	UFUNCTION()
	void OnPawnControllerChanged(APawn* InPawn, AController* InController);

	/** If the Pawn changed, we need to let all the sensors know. */
	virtual void OnPawnChanged(APawn* NewPawn, AController* InController);

	friend UMLAdapterSession;

	/** Set this agent's ID to its new ID. */
	void SetAgentID(FMLAdapter::FAgentID NewAgentID) { AgentID = NewAgentID; }

public:

	/** Retrieve all the sensor data from the last time Sense was called. */
	void GetObservations(FMLAdapterMemoryWriter& Ar);

	/** Get this agent's current config. This will not be accurate if the agent was spawned from a blueprint. */
	const FMLAdapterAgentConfig& GetConfig() const { return AgentConfig; }

	/** Setup this agent's avatar, sensors, and actuators. Typically used for agents spawned via RPC. */
	virtual void Configure(const FMLAdapterAgentConfig& NewConfig);

	/** Get the overall action space of this agent based on all its actuators. Used to determine the necessary output shape of the ML model. */
	virtual void GetActionSpaceDescription(FMLAdapterSpaceDescription& OutSpaceDesc) const;

	/** Get the overall observation space of this agent based on all its sensors. Used to determine the necessary input shape of the ML model. */
	virtual void GetObservationSpaceDescription(FMLAdapterSpaceDescription& OutSpaceDesc) const;

	/** Get the session that this agent belongs to. */
	UMLAdapterSession& GetSession();
	
	/** Returns true is the given avatar can be controlled by this agent. */
	virtual bool IsSuitableAvatar(AActor& InAvatar) const;

	/** Sets the avatar for this agent and all of its sensors and actuators. Registers callbacks. */
	virtual void SetAvatar(AActor* InAvatar);

	/** Get the avatar this agent is controlling. */
	AActor* GetAvatar() const { return Avatar; }

	/** Returns true if this agent has an avatar set. */
	bool IsReady() const { return Avatar != nullptr; }

	/** Enable/disable the action durations with the specified time duration in seconds. */
	void EnableActionDuration(bool bEnable, float DurationSeconds);

	/** Resets the action duration flag if it has elapsed. Returns false if not reset yet. Used with HasActionDurationElapsed by Manager. */
	bool TryResetActionDuration();

protected:
	virtual void ShutDownSensorsAndActuators();

protected:
	UPROPERTY(EditDefaultsOnly, Instanced, BlueprintReadOnly, Category = MLAdapter)
	TArray<TObjectPtr<UMLAdapterSensor>> Sensors;

	UPROPERTY(EditDefaultsOnly, Instanced, BlueprintReadOnly, Category = MLAdapter)
	TArray<TObjectPtr<UMLAdapterActuator>> Actuators;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = MLAdapter)
	TSubclassOf<AActor> AvatarClass;

	// If true, then agents won't be able to make a new decision until each action duration has elapsed.
	UPROPERTY(EditAnywhere, Category = MLAdapter)
	bool bEnableActionDuration = false;

	// How long should agents wait before they can change their action
	UPROPERTY(EditAnywhere, Category = MLAdapter)
	float ActionDurationSeconds = 0.1f;

	// How much time has the current action been executing
	UPROPERTY(VisibleInstanceOnly, Category = MLAdapter)
	float CurrentActionDuration = 0.f;

	UPROPERTY(VisibleInstanceOnly, Category = MLAdapter)
	bool bActionDurationElapsed = false;

	mutable FCriticalSection ActionDurationCS;

private:
	UPROPERTY()
	TObjectPtr<AActor> Avatar;

	UPROPERTY()
	TObjectPtr<AController> Controller;

	UPROPERTY()
	TObjectPtr<APawn> Pawn;

	FMLAdapter::FAgentID AgentID;

	FMLAdapterAgentConfig AgentConfig;

	/** True if the agent ever had a Pawn. Otherwise, false. */
	uint32 bEverHadAvatar : 1;

	/** True if we have callbacks registered. */
	uint32 bRegisteredForPawnControllerChange : 1;
};
