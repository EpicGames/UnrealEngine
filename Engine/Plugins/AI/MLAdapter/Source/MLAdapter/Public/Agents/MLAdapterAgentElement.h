// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "MLAdapterSpace.h"
#include "MLAdapterAgentElement.generated.h"


class UMLAdapterAgent;
struct FMLAdapterDescription;

/**
 * An agent element is any object that can be attached to an agent. Base class for UMLAdapterSensor and UMLAdapterActuator.
 */
UCLASS(abstract)
class MLADAPTER_API UMLAdapterAgentElement : public UObject
{
	GENERATED_BODY()
public:
	UMLAdapterAgentElement(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	virtual void PostInitProperties() override;

	/** Get the agent this element is attached to. */
	const UMLAdapterAgent& GetAgent() const;

	/** Get the ID of this element. */
	uint32 GetElementID() const { return ElementID; }

	/** Get nickname of this element. */
	const FString& GetNickname() const { return Nickname; }

	/** @return The owning agent's avatar. */
	AActor* GetAvatar() const;

	/**
	 * @return The pawn associated with the owning agent. If owning agent's avatar is a pawn then that gets retrieved,
	 * if not the function will check if it's a controller and if so retrieve its pawn.
	 */
	APawn* GetPawnAvatar() const;

	/**
	 * @return The controller associated with the owning agent. If owning agent's avatar is a controller then that
	 * gets retrieved, if not the function will check if it's a pawn and if so retrieve its controller.
	 */
	AController* GetControllerAvatar() const;

	/**
	 * Fetches both the pawn and the controller associated with the current agent.
	 * It's like both calling @see GetPawnAvatar and @see GetControllerAvatar.
	 * @return true if at least one of the fetched pair is non-null.
	 */
	bool GetPawnAndControllerAvatar(APawn*& OutPawn, AController*& OutController) const;

	virtual FString GetDescription() const { return Description; }

	/**
	 * Called before object's destruction. Can be called as part of new agent config application when old actuator get
	 * destroyed.
	 */
	virtual void Shutdown() {}
	
	void SetNickname(const FString& NewNickname) { Nickname = NewNickname; }
	virtual void Configure(const TMap<FName, FString>& Params);
	virtual void OnAvatarSet(AActor* Avatar) {}

	virtual void UpdateSpaceDef();
	const FMLAdapter::FSpace& GetSpaceDef() const { return *SpaceDef; }

#if WITH_GAMEPLAY_DEBUGGER
	virtual void DescribeSelfToGameplayDebugger(class FGameplayDebuggerCategory& DebuggerCategory) const;
#endif // WITH_GAMEPLAY_DEBUGGER

protected:
	virtual TSharedPtr<FMLAdapter::FSpace> ConstructSpaceDef() const PURE_VIRTUAL(UMLAdapterAgentElement::ConstructSpaceDef, return MakeShareable(new FMLAdapter::FSpace_Dummy()); );

protected:
	/** Can be queried by remote clients. */
	FString Description;

	TSharedRef<FMLAdapter::FSpace> SpaceDef;

	/** @note This is not a common counter, meaning Sensors and Actuators (for example) track the ID separately. */
	UPROPERTY()
	uint32 ElementID;

	/**
	 * User-configured name for this element, mostly for debugging purposes but comes in handy when fetching
	 * observation/action spaces descriptions. Defaults to UnrealEngine instance name.
	 */
	UPROPERTY()
	FString Nickname; 

#if WITH_GAMEPLAY_DEBUGGER
	/** Displayed in debugging tools and logging. */
	mutable FString DebugRuntimeString;
#endif // WITH_GAMEPLAY_DEBUGGER
};

struct FAgentElementSort
{
	bool operator()(const UMLAdapterAgentElement* A, const UMLAdapterAgentElement* B) const
	{
		return A && (!B || (A->GetElementID() < B->GetElementID()));
	}
	bool operator()(const UMLAdapterAgentElement& A, const UMLAdapterAgentElement& B) const
	{
		return (A.GetElementID() < B.GetElementID());
	}
};
