// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"
#include "UObject/SoftObjectPath.h"
#include "Agents/MLAdapterAgent.h"
#include "NNEModelData.h"
#include "NNERuntimeCPU.h"
#include "MLAdapterAgent_Inference.generated.h"

/**
 * Inference agents have a neural network that can process senses and output actuations via their Think method. You 
 * can create a blueprint of this class to easily wire-up an agent that functions entirely inside the Unreal Engine.
 * @see UMLAdapterNoRPCManager - we do not usually need to open the RPC interface if we are exclusively using inference agents.
 */
UCLASS(Blueprintable, EditInlineNew)
class MLADAPTER_API UMLAdapterAgent_Inference : public UMLAdapterAgent
{
	GENERATED_BODY()
public:
	/** Tries to load the neural network from the ModelDataPath. */
	virtual void PostInitProperties() override;

	/** Process observations from the sensors with the neural network and populate the actuators with data. */
	virtual void Think(const float DeltaTime) override;

	/** Reference to neural network asset data. */
	UPROPERTY(EditAnywhere, Category = MLAdapter)
	TObjectPtr<UNNEModelData> ModelData;

	/** The neural network controlling this agent. */
	TSharedPtr<UE::NNE::IModelInstanceCPU> Brain;

private:
	uint32 InputTensorSizeInBytes = 0;
	uint32 OutputTensorSizeInBytes = 0;
};
