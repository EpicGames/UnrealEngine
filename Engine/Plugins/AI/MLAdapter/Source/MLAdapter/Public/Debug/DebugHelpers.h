// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define NOT_IMPLEMENTED() ensureMsgf(false && "NotImplemented", TEXT("NotImplemented"))
