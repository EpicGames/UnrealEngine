// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU
#include "GameplayDebuggerCategory.h"
#include "MLAdapterTypes.h"


class UMLAdapterAgent;

class FGameplayDebuggerCategory_MLAdapter : public FGameplayDebuggerCategory
{
public:
	FGameplayDebuggerCategory_MLAdapter();

	virtual void CollectData(APlayerController* OwnerPC, AActor* DebugActor) override;
	virtual void DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext) override;

	static TSharedRef<FGameplayDebuggerCategory> MakeInstance();

protected:
	void ResetProps();

	void OnShowNextAgent();
	void OnRequestAvatarUpdate();
	void OnSetAvatarAsDebugAgent();

	void OnCurrentSessionChanged();
	void OnAgentAvatarChanged(UMLAdapterAgent& Agent, AActor* OldAvatar);
	void OnBeginAgentRemove(UMLAdapterAgent& Agent);

	void Init();

protected:
	AActor* CachedDebugActor;
	FMLAdapter::FAgentID CachedAgentID;
};

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
