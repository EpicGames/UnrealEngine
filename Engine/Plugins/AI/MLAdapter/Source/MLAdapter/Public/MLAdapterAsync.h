// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Templates/Function.h"
#include "Async/Async.h"


// sends InFunction to be called on the GameThread and waits for the result
template<typename RetType>
RetType CallOnGameThread(TFunction<RetType()> InFunction)
{
	TOptional<RetType> Result;
	TAtomic<bool> bCompleted(false);
	AsyncTask(ENamedThreads::GameThread, [&Result, &bCompleted, &InFunction]
	{
		Result = InFunction();
		bCompleted = true;
	});
	while (!bCompleted);
	return MoveTemp(Result.GetValue());
}

// fire (InFunction to be called on the GameThread) and forget
template<>
inline void CallOnGameThread(TFunction<void()> InFunction)
{
	AsyncTask(ENamedThreads::GameThread, InFunction);
}

