// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputCoreTypes.h"
#include "MLAdapterSpace.h"
#include "EnhancedPlayerInput.h"

namespace FMLAdapterInputHelper
{
	MLADAPTER_API void CreateInputMap(TArray<TTuple<FKey, FName>>& InterfaceKeys, TMap<FKey, int32>& FKeyToInterfaceKeyMap);

	MLADAPTER_API TSharedPtr<FMLAdapter::FSpace> ConstructEnhancedInputSpaceDef(const TArray<UInputAction*>& TrackedActions);
}