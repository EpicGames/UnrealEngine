// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "JsonObjectConverter.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Policies/CondensedJsonPrintPolicy.h"

namespace FMLAdapter
{
	template<typename StructType>
	FString StructArrayToJsonString(const TArray<StructType>& InStructArray)
	{
		FString OutJsonString = TEXT("[");
		FString TempArrayString;
		for (int32 ArrayIndex = 0; ArrayIndex < InStructArray.Num(); ArrayIndex++)
		{
			if (FJsonObjectConverter::UStructToFormattedJsonObjectString<TCHAR, TPrettyJsonPrintPolicy>(StructType::StaticStruct(), &InStructArray[ArrayIndex], TempArrayString, 0, 0))
			{
				if (ArrayIndex != (InStructArray.Num() - 1))
				{
					OutJsonString += TempArrayString + TEXT(',');
				}
			}
		}
		OutJsonString += TempArrayString + TEXT("]");

		return OutJsonString;
	}

	template<typename StructType>
	FString StructToJsonString(const StructType& InStruct)
	{
		FString RetString;
		FJsonObjectConverter::UStructToFormattedJsonObjectString<TCHAR, TCondensedJsonPrintPolicy>(
			StructType::StaticStruct(), &InStruct, RetString, 0, 0);
		return RetString;
	}

	template<typename StructType>
	bool JsonStringToStruct(const FString& JsonString, StructType& OutStruct)
	{
		return FJsonObjectConverter::JsonObjectStringToUStruct(JsonString, &OutStruct, 0, 0);
	}

	struct IJsonable
	{
		virtual FString ToJson() const PURE_VIRTUAL(IJsonable::ToJson(), return TEXT("Invalid");)
	};
}