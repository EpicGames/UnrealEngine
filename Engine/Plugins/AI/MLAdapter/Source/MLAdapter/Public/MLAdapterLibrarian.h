// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/SubclassOf.h"
#include "Sensors/MLAdapterSensor.h"
#include "Actuators/MLAdapterActuator.h"
#include "Agents/MLAdapterAgent.h"
#include "MLAdapterLibrarian.generated.h"

/**
 * The FMLAdapterLibrarian discovers all classes that derived from UMLAdapterAgent, UMLAdapterSensor, or UMLAdapterActuator. 
 * It provides functionality for finding those classes by name, so that remote clients can spawn them via their name.
 */
USTRUCT()
struct FMLAdapterLibrarian
{
	GENERATED_BODY()
	
	/**
	* Register all the base classes and derived classes that inherit from UMLAdapterAgent, UMLAdapterSensor, or
	* UMLAdapterActuator.
	*/
	void GatherClasses();

	/** Add the sensor class to the list of known sensors. */
	void RegisterSensorClass(const TSubclassOf<UMLAdapterSensor>& Class);

	/** Add the actuator class to the list of known actuators. */
	void RegisterActuatorClass(const TSubclassOf<UMLAdapterActuator>& Class);

	/** Add the agent class to the list of known agents. */
	void RegisterAgentClass(const TSubclassOf<UMLAdapterAgent>& Class);

	/** Add the given description to the mapped entry for the given function name. */
	void AddRPCFunctionDescription(const FName FunctionName, FString&& Description);

	/** Get a const iterator over the list of known sensors. */
	TMap<uint32, TSubclassOf<UMLAdapterSensor> >::TConstIterator GetSensorsClassIterator() const { return KnownSensorClasses.CreateConstIterator(); }

	/** Get a const iterator over the list of known actuators. */
	TMap<uint32, TSubclassOf<UMLAdapterActuator> >::TConstIterator GetActuatorsClassIterator() const { return KnownActuatorClasses.CreateConstIterator(); }

	/** Get a const iterator over the list of known agents. */
	TArray<TSubclassOf<UMLAdapterAgent> >::TConstIterator GetAgentsClassIterator() const { return KnownAgentClasses.CreateConstIterator(); }

	/**
	 * Gets the function description for the given function name.
	 * @param FunctionName The name of the function whose description we would like to get.
	 * @param OutDescription The description will be returned here if found.
	 * @return True if the description was found. Otherwise false.
	 */
	bool GetFunctionDescription(const FName FunctionName, FString& OutDescription) const;

	/**
	 * Gets the function description for the given function name.
	 * @param FunctionName The name of the function whose description we would like to get.
	 * @param OutDescription The description will be returned here if found.
	 * @return True if the description was found. Otherwise false.
	 */
	inline bool GetFunctionDescription(const FString& FunctionName, FString& OutDescription) const;

	/** Get a const iterator over the list of function descriptions. */
	TMap<FName, FString>::TConstIterator GetFunctionDescriptionsIterator() const { return RPCFunctionDescriptions.CreateConstIterator(); }

	/**
	 * Gets the description for the given sensor name.
	 * @param SensorName The name of the sensor whose description we would like to get.
	 * @param OutDescription The description will be returned here if found.
	 * @return True if the description was found. Otherwise false.
	 */
	bool GetSensorDescription(const FName SensorName, FString& OutDescription) const;

	/**
	 * Gets the description for the given actuator name.
	 * @param ActuatorName The name of the actuator whose description we would like to get.
	 * @param OutDescription The description will be returned here if found.
	 * @return True if the description was found. Otherwise false.
	 */
	bool GetActuatorDescription(const FName ActuatorName, FString& OutDescription) const;

	/**
	 * Find the agent class with the given class name. Can be the full name or the substring after "UMLAdapterAgent_".
	 * Will return UMLAdapterAgent::StaticClass() if not found.
	 */
	TSubclassOf<UMLAdapterAgent> FindAgentClass(const FName ClassName) const;

	/**
     * Find the sensor class with the given class name. Can be the full name or the substring after "UMLAdapterSensor_".
	 * Will return UMLAdapterSensor::StaticClass() if not found.
	 */
	TSubclassOf<UMLAdapterSensor> FindSensorClass(const FName ClassName) const;

	/**
	 * Find the actuator class with the given class name. Can be the full name or the substring after "UMLAdapterActuator_".
	 * Will return UMLAdapterActuator::StaticClass() if not found.
	 */
	TSubclassOf<UMLAdapterActuator> FindActuatorClass(const FName ClassName) const;

	/** Gets the librarian owned by the singleton UMLAdapterManager instance. */
	static const FMLAdapterLibrarian& Get();

protected:
	UPROPERTY()
	TMap<uint32, TSubclassOf<UMLAdapterSensor> > KnownSensorClasses;

	UPROPERTY()
	TMap<uint32, TSubclassOf<UMLAdapterActuator> > KnownActuatorClasses;

	UPROPERTY()
	TArray<TSubclassOf<UMLAdapterAgent> > KnownAgentClasses;

	TMap<FName, FString> RPCFunctionDescriptions;
};

//----------------------------------------------------------------------//
// inlines 
//----------------------------------------------------------------------//
bool FMLAdapterLibrarian::GetFunctionDescription(const FString& FunctionName, FString& OutDescription) const
{
	return GetFunctionDescription(FName(*FunctionName), OutDescription);
}