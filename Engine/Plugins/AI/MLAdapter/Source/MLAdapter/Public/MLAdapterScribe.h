// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include <vector>
#include <string>
#include <map>

/** Describes requested data in an rpclib-usable format. */
namespace FMLAdapterScribe
{
	std::vector<std::string> ToStringVector(const TArray<FString>& Array);
	std::vector<std::string> ToStringVector(const TArray<FName>& Array);
	std::vector<std::string> ListFunctions();
	std::map<std::string, uint32> ListSensorTypes();
	std::map<std::string, uint32> ListActuatorTypes();

	/**
	* Gets the description for the given name. Searches the Librarian for registered names in the following order:
	*	1. Functions
	*	2. Sensors
	*	3. Actuators
	* @return The description if found. Otherwise, returns string literal "Not Found".
	*/ 
	std::string GetDescription(std::string const& ElementName);
};