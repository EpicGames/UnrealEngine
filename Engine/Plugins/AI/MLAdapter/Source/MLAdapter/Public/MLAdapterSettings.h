// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Agents/MLAdapterAgent.h"
#include "Sessions/MLAdapterSession.h"
#include "Managers/MLAdapterManager.h"
#include "MLAdapterSettings.generated.h"


class UMLAdapterManager;
class UMLAdapterAgent;
class UMLAdapterSession;

#define GET_CONFIG_VALUE(a) (GetDefault<UMLAdapterSettings>()->a)

/**
 * Implements the settings for the MLAdapter plugin.
 */
UCLASS(config = Plugins, defaultconfig, DisplayName="MLAdapter")
class MLADAPTER_API UMLAdapterSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UMLAdapterSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	static TSubclassOf<UMLAdapterManager> GetManagerClass();
	static TSubclassOf<UMLAdapterSession> GetSessionClass();
	static TSubclassOf<UMLAdapterAgent> GetAgentClass();
	static uint16 GetDefaultRPCServerPort() { return GET_CONFIG_VALUE(DefaultRPCServerPort); }

protected:
	UPROPERTY(EditAnywhere, config, Category = MLAdapter, meta = (MetaClass = "/Script/MLAdapter.MLAdapterManager"))
	FSoftClassPath ManagerClass;

	UPROPERTY(EditAnywhere, config, Category = MLAdapter, meta = (MetaClass = "/Script/MLAdapter.MLAdapterSession"))
	FSoftClassPath SessionClass;

	UPROPERTY(EditAnywhere, config, Category = MLAdapter, meta = (MetaClass = "/Script/MLAdapter.MLAdapterAgent"))
	FSoftClassPath DefaultAgentClass;

	UPROPERTY(EditAnywhere, config, Category = MLAdapter)
	uint16 DefaultRPCServerPort = 15151;
};

#undef GET_CONFIG_VALUE