// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MLAdapterTypes.h"
#include "MLAdapterJson.h"
#include "MLAdapterSpace.generated.h"


UENUM()
enum class EMLAdapterSpaceType : uint8
{
	Discrete,
	MultiDiscrete,
	Box,
	Tuple,
	MAX
};
DECLARE_ENUM_TO_STRING(EMLAdapterSpaceType);


namespace FMLAdapter
{
	/** Defines the numerical space of a sensor, actuator, or agent. Similar to OpenAI Gym's spaces. */
	struct MLADAPTER_API FSpace : public IJsonable, public TSharedFromThis<FSpace>
	{
		virtual ~FSpace() {}
		EMLAdapterSpaceType Type = EMLAdapterSpaceType::MAX;

		virtual FString ToJson() const { return TEXT("{\"InvalidFSpaceType\": 0}"); }
		virtual int32 Num() const { return 0; }
	};

	/** A discrete space contains a countable number of values. */
	struct MLADAPTER_API FSpace_Discrete : public FSpace
	{
		uint32 Count;
		explicit FSpace_Discrete(uint32 InCount = 2);
		virtual FString ToJson() const override;
		virtual int32 Num() const override { return Count; }
	};

	/** Multiple options, each with separate discrete range */
	struct MLADAPTER_API FSpace_MultiDiscrete : public FSpace
	{
		TArray<uint32> Options;
		// simplified constructor creating InCount number of InValues-count options
		explicit FSpace_MultiDiscrete(uint32 InCount, uint32 InValues = 2); 
		// each element in InOptions defines number of possible values for each n-th option
		explicit FSpace_MultiDiscrete(std::initializer_list<uint32> InOptions);
		explicit FSpace_MultiDiscrete(const TArray<uint32>& InOptions);
		virtual FString ToJson() const override;
		virtual int32 Num() const override { return Options.Num(); }
	};

	/** A continuous space that contains a number of ranges defined by the shape, whose values will fall inside the Low to High range. */
	struct MLADAPTER_API FSpace_Box : public FSpace
	{
		TArray<uint32> Shape;
		float Low = -1.f;
		float High = 1.f;
		FSpace_Box();
		FSpace_Box(std::initializer_list<uint32> InShape, float InLow = -1.f, float InHigh = 1.f);
		virtual FString ToJson() const override;
		virtual int32 Num() const override;

		static TSharedPtr<FSpace> Vector3D(float Low = -1.f, float High = 1.f) { return MakeShareable(new FSpace_Box({ 3 }, Low, High)); }
		static TSharedPtr<FSpace> Vector2D(float Low = -1.f, float High = 1.f) { return MakeShareable(new FSpace_Box({ 2 }, Low, High)); }
	};

	/** A placeholder shape - typically returned in invalid scenarios. */
	struct MLADAPTER_API FSpace_Dummy : public FSpace_Box
	{
		FSpace_Dummy() : FSpace_Box({ 0 }) {}
		virtual int32 Num() const override { return 0; }
	};

	/** Container for multiple subspaces. */
	struct MLADAPTER_API FSpace_Tuple : public FSpace
	{
		TArray<TSharedPtr<FSpace> > SubSpaces;
		FSpace_Tuple();
		FSpace_Tuple(std::initializer_list<TSharedPtr<FSpace> > InitList);
		FSpace_Tuple(TArray<TSharedPtr<FSpace> >& InSubSpaces);
		virtual FString ToJson() const override;
		virtual int32 Num() const override;
	};

	struct FSpaceSerializeGuard
	{
		const TSharedRef<FMLAdapter::FSpace>& Space;
		FMLAdapterMemoryWriter& Ar;
		const int64 Tell;
		const int ElementSize;
		FSpaceSerializeGuard(const TSharedRef<FMLAdapter::FSpace>& InSpace, FMLAdapterMemoryWriter& InAr,  const int InElementSize = sizeof(float))
			: Space(InSpace), Ar(InAr), Tell(InAr.Tell()), ElementSize(InElementSize)
		{}
		~FSpaceSerializeGuard()
		{
			ensure(FMath::Abs(Ar.Tell() - Tell) == Space->Num() * ElementSize);
		}
	};

}


struct FMLAdapterDescription
{
	static bool FromJson(const FString& JsonString, FMLAdapterDescription& OutInstance);
	FString ToJson() const;

	FMLAdapterDescription& Add(FString Key, FString Element) { Data.Add(TPair<FString, FString>(Key, Element)); return(*this); }
	FMLAdapterDescription& Add(FString Key, const FMLAdapterDescription& Element) { Add(Key, Element.ToJson()); return(*this); }
	FMLAdapterDescription& Add(FString Key, const int Element) { Add(Key, FString::FromInt(Element)); return(*this); }
	FMLAdapterDescription& Add(FString Key, const float Element) { Add(Key, FString::SanitizeFloat(Element)); return(*this); }
	FMLAdapterDescription& Add(const FMLAdapter::FSpace& Space) { PrepData.Add(Space.ToJson()); return(*this); }

	/** Used in loops to optimize memory use */
	void Reset() { Data.Reset(); PrepData.Reset(); }

	bool IsEmpty() const { return (Data.Num() == 0); }

protected:
	TArray<TPair<FString, FString>> Data;
	TArray<FString> PrepData;
};


struct FMLAdapterSpaceDescription
{
	typedef TPair<FString, FMLAdapterDescription> ValuePair;
	FString ToJson() const;

	FMLAdapterSpaceDescription& Add(FString Key, FMLAdapterDescription&& Element) { Data.Add(ValuePair(Key, Element)); return(*this); }
	FMLAdapterSpaceDescription& Add(FString Key, FMLAdapterDescription Element) { Data.Add(ValuePair(Key, Element)); return(*this); }

protected:
	TArray<ValuePair> Data;
};
