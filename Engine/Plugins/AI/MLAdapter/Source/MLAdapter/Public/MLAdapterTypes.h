// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

class AActor;
class AController;

DECLARE_LOG_CATEGORY_EXTERN(LogMLAdapter, Log, All);

#ifndef DEFINE_ENUM_TO_STRING
#define DEFINE_ENUM_TO_STRING(EnumType, EnumPackage) FString EnumToString(const EnumType Value) \
{ \
	static const UEnum* TypeEnum = FindObject<UEnum>(nullptr, TEXT(EnumPackage) TEXT(".") TEXT(#EnumType)); \
	return TypeEnum->GetNameStringByIndex(static_cast<int32>(Value)); \
}
#endif //DEFINE_ENUM_TO_STRING
#ifndef DECLARE_ENUM_TO_STRING
#define DECLARE_ENUM_TO_STRING(EnumType) FString EnumToString(const EnumType Value)
#endif // DECLARE_ENUM_TO_STRING

namespace FMLAdapter
{
	typedef uint32 FAgentID;
	static const FAgentID InvalidAgentID = FAgentID(-1);
	static const uint32 InvalidSensorID = 0;
	static const uint32 InvalidActuatorID = 0;

	/**
	 *	MLAdapter-flavored new object creation. We're using this with all the objects 
	 *	we intend to access from outside the game-thread (most notably from the 
	 *	rpc calls). This gives us more control over objects' life cycle.
	 *
	 *	Note: objects created this way won't go away even if there's nothing
	 *	referencing them until we clear the EInternalObjectFlags::Async flag 
	 */
	template<class T>
	T* NewObject(UObject* Outer)
	{
		T* Object = ::NewObject<T>(Outer);
		Object->SetInternalFlags(EInternalObjectFlags::Async);
		return Object;
	}

	template<class T>
	T* NewObject(UObject* Outer, UClass* Class, FName Name = NAME_None, EObjectFlags Flags = RF_NoFlags, UObject* Template = nullptr, bool bCopyTransientsFromClassDefaults = false, FObjectInstancingGraph* InInstanceGraph = nullptr)
	{
		T* Object = ::NewObject<T>(Outer, Class, Name, Flags, Template, bCopyTransientsFromClassDefaults, InInstanceGraph);
		Object->SetInternalFlags(EInternalObjectFlags::Async);
		return Object;
	}	

	AController* ActorToController(AActor& Actor);
}

namespace rpc {
#if WITH_RPCLIB
class server;
#else
// mock server
class server
{
public:
	server(uint16) {}
	void stop() {}
	void async_run(uint16) {}
};
#endif // WITH_RPCLIB
}
typedef rpc::server FRPCServer;

struct FMLAdapterMemoryWriter : FMemoryWriter
{
	FMLAdapterMemoryWriter(TArray<uint8>& InBytes, bool bIsPersistent = false, bool bSetOffset = false, const FName InArchiveName = NAME_None)
		: FMemoryWriter(InBytes, bIsPersistent, bSetOffset, InArchiveName)
	{}

#if WITH_RPCLIB && 0
	template <typename MSGPACK_OBJECT>
	void msgpack_object(MSGPACK_OBJECT *o, clmdep_msgpack::zone &z) const {
		std::vector<uint8> vec(Bytes.Num());
		FMemory::Memcpy(vec.data(), Bytes.GetData(), Bytes.Num() * sizeof(uint8));
		clmdep_msgpack::type::make_define_array(vec).msgpack_object(o, z);
	}
#endif // WITH_RPCLIB
};


struct FMLAdapterMemoryReader : FMemoryReader
{
	FMLAdapterMemoryReader(const TArray<uint8>& InBytes, bool bIsPersistent = false)
		: FMemoryReader(InBytes, bIsPersistent)
	{}
};

