// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"
#include "Delegates/IDelegateInstance.h"
#include "Logging/LogMacros.h"
#include "Tickable.h"
#include "Engine/World.h"
#include "Sessions/MLAdapterSession.h"
#include "MLAdapterLibrarian.h"
#include "MLAdapterManager.generated.h"

namespace rpc { class server; }
namespace FMLAdapterConsoleCommands { struct FHelper; }
class UMLAdapterSession;
class UMLAdapterActuator;
class UMLAdapterSensor;
class UMLAdapterAgent;
class AGameModeBase;
class APlayerController;
class UWorld;

UENUM()
enum class EMLAdapterServerMode : uint8
{
	Invalid,
	Server,
	Client,
	Standalone, // this applies both to Standalone games as well as PIE
	AutoDetect,
};

/**
 * The manager of the MLAdapter system. Sets up the RPC server for communication with remote client. Based on settings,
 * creates and ticks the UMLAdapterSession. Tells the UMLAdapterLibrarian to gather its classes.
 * A singleton instance is setup automatically during OnPostEngineInit if this plugin is included.
 */
UCLASS(Transient)
class MLADAPTER_API UMLAdapterManager : public UObject, public FTickableGameObject, public FSelfRegisteringExec
{
	GENERATED_BODY()
public:
	using FRPCFunctionBind = TFunction<void(FRPCServer&/*RPCServer*/)>;
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGenericRPCServerDelegate, FRPCServer& /*Server*/);
	DECLARE_MULTICAST_DELEGATE(FOnGenericEvent);
	
	UMLAdapterManager(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	virtual void BeginDestroy() override;
	virtual void PostInitProperties() override;

	// FTickableGameObject begin
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;
	virtual bool IsTickableWhenPaused() const override { return true; }
	virtual bool IsTickableInEditor() const override { return true; }
	virtual ETickableTickType GetTickableTickType() const override;
	virtual bool IsTickable() const override;
	// FTickableGameObject end

	// FExec begin
	virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	// FExec end

	/** Binds the manager's world, game mode, and editor delegates - called during PostInitProperties. */
	virtual void BindToDelegates();

	/** Cleans up all the manager's delegates - called during BeginDestroy. */
	virtual void CleanUpDelegates();

	/** Starts the RPC server if the world is a game world */
	virtual void OnPostWorldInit(UWorld* World, const UWorld::InitializationValues);

	/** Closes the current session. */
	virtual void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);

	virtual void OnBeginPIE(const bool bIsSimulating);
	virtual void OnEndPIE(const bool bIsSimulating);

	/** Notifies the session that the game mode has initialized. */
	virtual void OnGameModeInitialized(AGameModeBase* GameMode);

	/** Notifies the session a player has connected. */
	virtual void OnGameModePostLogin(AGameModeBase* GameMode, APlayerController* NewPlayer);

	/** @note This might not get called at all if the project's game mode doesn't extend AGameMode. */
	virtual void OnGameModeMatchStateSet(FName MatchState);

	/**
	 * Starts a new RPC server. If a server is already running, it will be shut down before the new instance gets created.
	 * @param ServerThreads best set at the number of external clients that are going to be connecting
	 */
	virtual void StartServer(uint16 Port, EMLAdapterServerMode InMode = EMLAdapterServerMode::AutoDetect, uint16 ServerThreads = 1);
	
	/** Stop the RPC server. */
	virtual void StopServer();

	/** True if there is an RPC server currently running. */
	virtual bool IsRunning() const;

	/** "Server" in this context means UnrealEngine game server, not RPC server. */
	virtual void ConfigureAsServer(FRPCServer& Server);

	/** "Client" in this context means UnrealEngine game client, not RPC client. */
	virtual void ConfigureAsClient(FRPCServer& Server);

	/** Essentially calls both the server and client versions. */
	virtual void ConfigureAsStandalone(FRPCServer& Server);

	/** If given World doesn't have an AI system, this call results in creating one. */
	virtual void EnsureAISystemPresence(UWorld& World);

	/** If given World doesn't have a Navigation system instance, this call results in creating one. */
	virtual void EnsureNavigationSystemPresence(UWorld& World);

	virtual UMLAdapterSession* CreateNewSession();
	virtual void SetSession(UMLAdapterSession* NewSession);
	virtual void CloseSession(UMLAdapterSession& InSession);

	/** Returns the current session. If one doesn't exist, it gets created. */
	virtual UMLAdapterSession& GetSession();
	bool HasSession() const { return IsValid(Session) && (Session->IsUnreachable() == false); }

	/** Register a sensor class with this manager's librarian. */
	void RegisterSensorClass(const TSubclassOf<UMLAdapterSensor>& Class) { Librarian.RegisterSensorClass(Class); }

	/** Register an actuator class with this manager's librarian. */
	void RegisterActuatorClass(const TSubclassOf<UMLAdapterActuator>& Class) { Librarian.RegisterActuatorClass(Class); }

	/** Register an agent class with this manager's librarian. */
	void RegisterAgentClass(const TSubclassOf<UMLAdapterAgent>& Class) { Librarian.RegisterAgentClass(Class); }

	virtual void ResetWorld();
	void SetManualWorldTickEnabled(bool bEnable);

	FOnGenericRPCServerDelegate& GetOnAddClientFunctions() { return OnAddClientFunctions; }
	FOnGenericRPCServerDelegate& GetOnAddServerFunctions() { return OnAddServerFunctions; }

	/** Get the current manager instance. */
	inline static UMLAdapterManager& Get();

	/** Returns true if the manager instance exists. */
	inline static bool IsReady();

	/** Get this manager's librarian. */
	const FMLAdapterLibrarian& GetLibrarian() const { return Librarian; }

	/** Returns true if this manager is not being ticked manually by the remote client. */
	bool IsWorldRealTime() const { return (bTickWorldManually == false); }

	FOnGenericEvent& GetOnCurrentSessionChanged() { return OnCurrentSessionChanged; }

protected:

	void AddCommonFunctions(FRPCServer& Server);

	virtual bool ShouldInitForWorld(const UWorld& World) const;

protected:	
	friend struct FMLAdapterConsoleCommands::FHelper;

	UPROPERTY()
	TObjectPtr<UMLAdapterSession> Session;

	UPROPERTY()
	TObjectPtr<UWorld> LastActiveWorld;

	UPROPERTY()
	FMLAdapterLibrarian Librarian;

	FOnGenericRPCServerDelegate OnAddClientFunctions;
	FOnGenericRPCServerDelegate OnAddServerFunctions;

	FOnGenericEvent OnCurrentSessionChanged;

	EMLAdapterServerMode RequestedFunctionMode = EMLAdapterServerMode::Invalid;
	EMLAdapterServerMode CurrentFunctionMode = EMLAdapterServerMode::Invalid;
	uint16 CurrentPort = 0;
	uint16 CurrentServerThreads = 1;

	TArray<uint8> Data;

	float WorldFPS = 20.f;

	uint32 bCommonFunctionsAdded : 1;
	uint32 bTickWorldManually : 1;
	
	/** is the manager is in 'manual ticking mode' (where external client is 
	 * responsible for progressing the world sim by calling 'request_world_tick' 
	 *	function) the simulation will progress by StepsRequested ticks before pausing */
	int32 StepsRequested = 0;

	FDelegateHandle OnPostWorldInitializationHandle;
	FDelegateHandle OnWorldCleanupHandle;
	FDelegateHandle OnGameModeInitializedHandle;
	FDelegateHandle OnGameModePostLoginHandle;
	FDelegateHandle OnGameModeMatchStateSetHandle;
#if WITH_EDITORONLY_DATA
	FDelegateHandle BeginPIEHandle;
	FDelegateHandle EndPIEHandle;
#endif // WITH_EDITORONLY_DATA

	static UMLAdapterManager* ManagerInstance;
public:
	static void RecreateManagerInstance();

	static FOnGenericEvent OnPostInit;
};

//----------------------------------------------------------------------//
// inlines 
//----------------------------------------------------------------------//
UMLAdapterManager& UMLAdapterManager::Get()
{ 
	// the only way for this check to fail is to call it too soon.
	check(ManagerInstance);  
	return *ManagerInstance; 
}

bool UMLAdapterManager::IsReady()
{ 
	return (ManagerInstance != nullptr); 
}