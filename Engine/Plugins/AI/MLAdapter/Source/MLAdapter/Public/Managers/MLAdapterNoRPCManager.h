// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Managers/MLAdapterManager.h"
#include "MLAdapterNoRPCManager.generated.h"

/**
 * UMLAdapterNoRPCManager won't start the RPC server and will immediately start a session and spawn the default agent. This is
 * intended to be used when there will only be a single inference agent that exists for the life of the game.
 */
UCLASS()
class MLADAPTER_API UMLAdapterNoRPCManager : public UMLAdapterManager
{
	GENERATED_BODY()
public:
	virtual void OnPostWorldInit(UWorld* World, const UWorld::InitializationValues) override;
};
