// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

#if WITH_RPCLIB
#include "rpclib_includes.h"

#define UE_VIRTUAL_MSGPACK_DEFINE(...)                                                \
    virtual void msgpack_object(RPCLIB_MSGPACK::object *o, clmdep_msgpack::zone &z)    \
        const {                                                                \
        clmdep_msgpack::type::make_define_array(__VA_ARGS__)                   \
            .msgpack_object(o, z);                                             \
    }

#define UE_MSGPACK_DEFINE(...)                                                \
    void msgpack_object(RPCLIB_MSGPACK::object *o, clmdep_msgpack::zone &z)    \
        const {                                                                \
        clmdep_msgpack::type::make_define_array(__VA_ARGS__)                   \
            .msgpack_object(o, z);                                             \
    }

#else // WITH_RPCLIB

#define UE_MSGPACK_DEFINE(...)
#define UE_VIRTUAL_MSGPACK_DEFINE(...)
#define MSGPACK_DEFINE_ARRAY(...)

#endif // WITH_RPCLIB

// helper macro for just binding a simple function. Not recommended for elaborate
// lambdas since using this macro interferes with debugging (unable to break in 
// the lambdas body).
#define UE_RPC_BIND(FuncName, FuncBody) TEXT(FuncName), [this](FRPCServer& Serv) { Serv.bind(FuncName, FuncBody); }
#define FSTRING_TO_STD(a) (std::string(TCHAR_TO_UTF8(*a)))

#include <vector>

namespace FMLAdapter
{
	template<typename T>
	TArray<T> VectorToArray(const std::vector<T> InVector)
	{
		return TArray<T>(InVector.data(), InVector.size());
	}
}