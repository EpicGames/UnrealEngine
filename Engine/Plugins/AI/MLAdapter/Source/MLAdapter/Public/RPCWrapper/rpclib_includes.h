// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_RPCLIB

#pragma push_macro("check")
#undef check

// we define 'nil' since on Macs we treat C++ as Objective-C++
#pragma push_macro("nil")
#undef nil

#pragma warning(disable:4005)
#pragma warning(disable:4668)

#ifdef __clang__
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wundef"
#pragma clang diagnostic ignored "-Wshadow"
#endif

#ifndef RPCLIB_MSGPACK
#define RPCLIB_MSGPACK clmdep_msgpack
#endif /* ifndef RPCLIB_MSGPACK */

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#include "rpc/client.h"
#include "rpc/server.h"
#include "rpc/this_handler.h"
#include "rpc/msgpack.hpp"

#ifdef __clang__
#pragma clang diagnostic pop
#endif

#pragma warning(default:4005)
#pragma warning(default:4668)

#pragma pop_macro("check")
#pragma pop_macro("nil")

#endif // WITH_RPCLIB
