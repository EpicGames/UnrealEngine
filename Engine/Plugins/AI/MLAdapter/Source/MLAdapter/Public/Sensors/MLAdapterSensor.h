// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Agents/MLAdapterAgentElement.h"
#include "MLAdapterTypes.h"
#include "MLAdapterSensor.generated.h"


class UMLAdapterAgent;
struct FMLAdapterDescription;

/** Controls the frequency that a sensor should tick at. */
UENUM()
enum class EMLAdapterTickPolicy : uint8
{
	EveryTick,
	EveryXSeconds,
	EveryNTicks,
	Never
};

/** Allows an agent to sense information about the game world or itself. */
UCLASS(Abstract, Blueprintable, EditInlineNew)
class MLADAPTER_API UMLAdapterSensor : public UMLAdapterAgentElement
{
    GENERATED_BODY()

public:
    UMLAdapterSensor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	virtual void PostInitProperties() override;

	/**
	 * Called before actuator's destruction. Can be called as part of new agent config application when old actuator
	 * get destroyed.
	 */
	virtual void Configure(const TMap<FName, FString>& Params) override;

	virtual void OnAvatarSet(AActor* Avatar) override;

	// AIGym leftovers. Potentially to be removed 
	FMLAdapter::FAgentID GetAgentID() const { return AgentID; }
	const UMLAdapterAgent& GetAgent() const;
	bool IsConfiguredForAgent(const UMLAdapterAgent& Agent) const;
	bool IsPolling() const { return bIsPolling; }

	/** True if config was successful. Only then will the sensor instance be added to agent's active sensors. */
	virtual bool ConfigureForAgent(UMLAdapterAgent& Agent);

	void OnPawnChanged(APawn* OldPawn, APawn* NewPawn);

	/** Called for every sense, regardless of whether it's a polling-type or not. */
	void Sense(const float DeltaTime);

	/** 
	 *	@param bTransfer if set to true the Sensor is expected to clear all stored
	 *		observations after copying/moving them to OutObservations
	 */
	virtual void GetObservations(FMLAdapterMemoryWriter& Ar) PURE_VIRTUAL(UMLAdapterSensor::GetObservations, );

protected:

	/** Clean up any references to the Pawn or its components. */
	virtual void ClearPawn(APawn& InPawn);

	/** Called from Sense based on TickPolicy. */
	virtual void SenseImpl(const float DeltaTime) {}

protected:
	FMLAdapter::FAgentID AgentID;

	/** @todo this is not currently referenced anywhere */
	UPROPERTY(EditDefaultsOnly, Category=MLAdapter)
	uint32 bRequiresPawn : 1;

	/** @todo this is not currently referenced anywhere */
	UPROPERTY(EditDefaultsOnly, Category = MLAdapter)
	uint32 bIsPolling : 1;

	/** Determines what frequency this sensor ticks at. */
	UPROPERTY(EditDefaultsOnly, Category = MLAdapter)
	EMLAdapterTickPolicy TickPolicy;

	struct FTicksOrSeconds
	{
		union 
		{
			int32 Ticks;
			float Seconds;
		};
		FTicksOrSeconds() : Ticks(0) {}
	};
	FTicksOrSeconds TickEvery;

	/**  This needs to be locked anytime the sensor is accessing shared state internally. */
	mutable FCriticalSection ObservationCS;
private:
	int32 AccumulatedTicks;
	float AccumulatedSeconds;
};	
