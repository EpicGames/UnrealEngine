// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sensors/MLAdapterSensor.h"
#include "MLAdapterSensor_AIPerception.generated.h"


class UAIPerceptionComponent;
class AActor;

/**
 *	When applied to a player controller will create an AIPerception component for 
 *	that player and plug it into the AIPerceptionSystem. The sensor will report 
 *	information gathered by the perception system on the behalf of this agent.
 *	@see UAIPerceptionComponent::ProcessStimuli
 *
 *	Note that the world needs to be configured to allow AI Systems to be created
 *	@see Server.Configure mz@todo replace with the proper reference
 */
UCLASS(Blueprintable)
class MLADAPTER_API UMLAdapterSensor_AIPerception : public UMLAdapterSensor
{
	GENERATED_BODY()
public:
	enum class ESortType : uint8
	{
		Distance, 
		InFrontness
	};
	struct FTargetRecord
	{
		FRotator HeadingRotator;
		FVector HeadingVector;
		float Distance;
		int32 ID;
		// non essential, helper/debug
		float HeadingDot;
		TWeakObjectPtr<AActor> Target;

		FTargetRecord() : HeadingRotator(EForceInit::ForceInitToZero), HeadingVector(EForceInit::ForceInitToZero), Distance(0.f), ID(0), HeadingDot(-1.f) {}
	};

	UMLAdapterSensor_AIPerception(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual void GetViewPoint(AActor& Avatar, FVector& POVLocation, FRotator& POVRotation) const;

protected:
	virtual void Configure(const TMap<FName, FString>& Params) override;
	virtual void GetObservations(FMLAdapterMemoryWriter& Ar) override;
	virtual TSharedPtr<FMLAdapter::FSpace> ConstructSpaceDef() const override;
	virtual void UpdateSpaceDef() override;
	virtual void OnAvatarSet(AActor* Avatar) override;

	virtual void SenseImpl(const float DeltaTime) override;

#if WITH_GAMEPLAY_DEBUGGER
	virtual void DescribeSelfToGameplayDebugger(FGameplayDebuggerCategory& DebuggerCategory) const override;
#endif // WITH_GAMEPLAY_DEBUGGER

	inline float Sanify(const float DegreeAngles) const
	{
		return (DegreeAngles < -180.f) 
			? (DegreeAngles + 360.f)
			: (DegreeAngles > 180.f) 
				? (DegreeAngles - 360.f) 
				: DegreeAngles;
	}
	inline FRotator Sanify(const FRotator Rotator) const
	{
		ensureMsgf(Rotator.Roll == 0.f, TEXT("UMLAdapterSensor_AIPerception is expected to deal only with 0-roll rotators"));
		return FRotator(Sanify(Rotator.Pitch), Sanify(Rotator.Yaw), 0.f);
	}

protected:
	UPROPERTY()
	TObjectPtr<UAIPerceptionComponent> PerceptionComponent;

	/**
	 *	When set to true will only gather perception "delta" meaning consecutive
	 *	updates will consist of new perception information. Defaults to "false" 
	 *	which means that every update all of data contained by the PerceptionComponent 
	 *	will be "sensed"
	 */
	UPROPERTY()
	bool bSenseOnlyChanges = false;

	float PeripheralVisionAngleDegrees;
	float MaxStimulusAge;
	uint16 TargetsToSenseCount;
	ESortType TargetsSortType;
	TArray<FTargetRecord> CachedTargets;
	uint32 bVectorMode : 1;
};
