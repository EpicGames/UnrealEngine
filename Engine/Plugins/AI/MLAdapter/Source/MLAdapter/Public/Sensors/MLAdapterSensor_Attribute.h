// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sensors/MLAdapterSensor.h"
#include "MLAdapterTypes.h"
#include "MLAdapterSensor_Attribute.generated.h"


class UAttributeSet;
struct FGameplayAttributeData;


UCLASS(Blueprintable)
class MLADAPTER_API UMLAdapterSensor_Attribute : public UMLAdapterSensor
{
	GENERATED_BODY()
public:
	UMLAdapterSensor_Attribute(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual bool ConfigureForAgent(UMLAdapterAgent& Agent) override;
	virtual void Configure(const TMap<FName, FString>& Params) override;

protected:
	virtual TSharedPtr<FMLAdapter::FSpace> ConstructSpaceDef() const override;
	virtual void UpdateSpaceDef() override;
	virtual void OnAvatarSet(AActor* Avatar) override;
	virtual void SenseImpl(const float DeltaTime) override;
	virtual void GetObservations(FMLAdapterMemoryWriter& Ar) override;

	void SetAttributes(TArray<FString>& InAttributeNames);
	void BindAttributes(AActor& Actor);
protected:
	UPROPERTY()
	TObjectPtr<UAttributeSet> AttributeSet;

	TArray<FName> AttributeNames;
	// valid only as long as AttributeSet != nullptr
	TArray<FGameplayAttributeData*> Attributes;

	TArray<float> Values;
};
