// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Sensors/MLAdapterSensor.h"
#include "Engine/TextureRenderTarget2D.h"
#include "MLAdapterSensor_Camera.generated.h"


class UMLAdapterAgent;
class USceneCaptureComponent2D;
class UTextureRenderTarget2D;
class UGameViewportClient;

/** Observing player's camera */
UCLASS(Blueprintable)
class MLADAPTER_API UMLAdapterSensor_Camera : public UMLAdapterSensor
{
	GENERATED_BODY()
public:
	UMLAdapterSensor_Camera(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual bool ConfigureForAgent(UMLAdapterAgent& Agent) override;
	virtual void Configure(const TMap<FName, FString>& Params) override;
	virtual TSharedPtr<FMLAdapter::FSpace> ConstructSpaceDef() const override;
	virtual void GetObservations(FMLAdapterMemoryWriter& Ar) override;

protected:
	virtual void SenseImpl(const float DeltaTime) override;

	virtual void OnAvatarSet(AActor* Avatar) override;
	virtual void ClearPawn(APawn& InPawn) override;

	void HandleScreenshotData(int32 InSizeX, int32 InSizeY, const TArray<FColor>& InImageData);

protected:
	UPROPERTY(EditAnywhere, Category = MLAdapter, meta=(UIMin=1, ClampMin=1))
	uint32 Width;

	UPROPERTY(EditAnywhere, Category = MLAdapter, meta=(UIMin=1, ClampMin=1))
	uint32 Height;

	UPROPERTY(EditAnywhere, Category = MLAdapter)
	uint32 bShowUI : 1;

	UPROPERTY(Transient)
	TObjectPtr<USceneCaptureComponent2D> CaptureComp;

	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> RenderTarget2D;

	UPROPERTY()
	TObjectPtr<UGameViewportClient> CachedViewportClient;

	ETextureRenderTargetFormat RenderTargetFormat = ETextureRenderTargetFormat::RTF_RGBA16f;
	ESceneCaptureSource CaptureSource = SCS_FinalColorLDR;
	uint8 CameraIndex = 0;

	FDelegateHandle ScreenshotDataCallbackHandle;
	
public: // tmp, just for prototyping
	TArray<FLinearColor> LastTickData;
};

