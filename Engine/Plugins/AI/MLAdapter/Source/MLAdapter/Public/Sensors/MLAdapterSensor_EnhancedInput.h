// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sensors/MLAdapterSensor.h"
#include "InputAction.h"
#include "EnhancedInputComponent.h"
#include "MLAdapterSensor_EnhancedInput.generated.h"


UCLASS(Blueprintable)
class MLADAPTER_API UMLAdapterSensor_EnhancedInput : public UMLAdapterSensor
{
	GENERATED_BODY()

protected:
	virtual void Configure(const TMap<FName, FString>& Params) override;
	virtual void GetObservations(FMLAdapterMemoryWriter& Ar) override;
	virtual void UpdateSpaceDef() override;
	virtual TSharedPtr<FMLAdapter::FSpace> ConstructSpaceDef() const override;

	/** The Avatar here is expected to be a player-controlled APawn or an APlayerController */
	virtual void OnAvatarSet(AActor* Avatar) override;

	TArray<float> InputState;

	TMap<FString, int32> InputStateIndices;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = MLAdapter)
	TArray<TObjectPtr<UInputAction>> TrackedActions;

	UPROPERTY(BlueprintReadOnly, Category = MLAdapter)
	TObjectPtr<UEnhancedInputComponent> InputComponent;

	UFUNCTION()
	void OnInputAction(const FInputActionInstance& ActionInstance);
};
