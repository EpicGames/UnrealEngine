// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sensors/MLAdapterSensor.h"
#include "InputCoreTypes.h"
#include "KeyState.h"
#include <vector>
#include "MLAdapterSensor_Input.generated.h"


struct FInputKeyEventArgs; 
class FViewport;
class UGameViewportClient;


/**
 *	Note that this sensor doesn't buffer input state between GetObservations call
 *	@todo a child class could easily do that by overriding OnInputKey/OnInputAxis and GetObservations
 */
UCLASS(Blueprintable)
class MLADAPTER_API UMLAdapterSensor_Input : public UMLAdapterSensor
{
	GENERATED_BODY()
public:
	UMLAdapterSensor_Input(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	
protected:
	virtual void OnInputAxis(FViewport* InViewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime, int32 NumSamples, bool bGamepad);
	virtual void OnInputKey(const FInputKeyEventArgs& EventArgs);

	virtual void Configure(const TMap<FName, FString>& Params) override;
	virtual void GetObservations(FMLAdapterMemoryWriter& Ar) override;
	virtual void UpdateSpaceDef() override; 
	virtual TSharedPtr<FMLAdapter::FSpace> ConstructSpaceDef() const override;
	virtual void OnAvatarSet(AActor* Avatar);

	UPROPERTY(EditDefaultsOnly, Category = MLAdapter)
	TObjectPtr<UGameViewportClient> GameViewport;

	UPROPERTY(EditDefaultsOnly, Category = MLAdapter)
	uint32 bRecordKeyRelease : 1;
	
	TArray<float> InputState;

	// stores (FKey, ActionName pairs). The order is important since FKeyToInterfaceKeyMap refers to it. 
	TArray<TTuple<FKey, FName>> InterfaceKeys;

	TMap<FKey, int32> FKeyToInterfaceKeyMap;
};
