// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sensors/MLAdapterSensor.h"
#include "MLAdapterTypes.h"
#include "MLAdapterSensor_Movement.generated.h"

/** Allows an agent to sense its avatar's location and velocity. */
UCLASS(Blueprintable)
class MLADAPTER_API UMLAdapterSensor_Movement : public UMLAdapterSensor
{
	GENERATED_BODY()
public:
	UMLAdapterSensor_Movement(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual bool ConfigureForAgent(UMLAdapterAgent& Agent) override;
	virtual void Configure(const TMap<FName, FString>& Params) override;
	virtual TSharedPtr<FMLAdapter::FSpace> ConstructSpaceDef() const override;

protected:
	virtual void OnAvatarSet(AActor* Avatar) override;
	virtual void SenseImpl(const float DeltaTime) override;
	virtual void GetObservations(FMLAdapterMemoryWriter& Ar) override;

protected:
	UPROPERTY()
	uint32 bAbsoluteLocation : 1;
	
	UPROPERTY()
	uint32 bAbsoluteVelocity : 1;

	UPROPERTY()
	FVector RefLocation;

	UPROPERTY()
	FVector RefVelocity;

	UPROPERTY()
	FVector CurrentLocation;

	UPROPERTY()
	FVector CurrentVelocity;
};
