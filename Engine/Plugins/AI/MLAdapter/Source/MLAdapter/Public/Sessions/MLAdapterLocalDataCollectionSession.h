// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Sessions/MLAdapterSession.h"
#include "Engine/EngineTypes.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Misc/DateTime.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "MLAdapterLocalDataCollectionSession.generated.h"

/**
 * Collects data from agents' sensors and writes them to a file for offline processing. Only works with a locally
 * controlled, single-player game.
 */
UCLASS(Blueprintable, EditInlineNew)
class MLADAPTER_API UMLAdapterLocalDataCollectionSession : public UMLAdapterSession
{
	GENERATED_BODY()
public:
	virtual void PostInitProperties() override;

	virtual void OnPostWorldInit(UWorld& World) override;

	UFUNCTION()
	virtual void OnPawnControllerChanged(APawn* InPawn, AController* InController);

	virtual void Tick(float DeltaTime) override;

	virtual void Close() override;

	UPROPERTY(EditDefaultsOnly, Category = MLAdapter)
	FDirectoryPath FilePath;

	UPROPERTY(EditDefaultsOnly, Category = MLAdapter)
	FString FileName;

	UPROPERTY(EditDefaultsOnly, Category = MLAdapter)
	bool bPrefixOutputFilenameWithTimestamp;

private:
	FString OutputFilePath;

	UPROPERTY()
	TWeakObjectPtr<UMLAdapterAgent> PlayerControlledAgent;
};
