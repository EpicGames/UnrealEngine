// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "MLAdapterTypes.h"
#include "Tickable.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "MLAdapterSession.generated.h"


class AGameModeBase;
class UMLAdapterAgent;
struct FMLAdapterAgentConfig;
class APawn;
class AController;
class APlayerController;
class UGameInstance;


enum class EMLAdapterSimState : uint8
{
	BootingUp,
	InProgress,
	Finished,
};


/**
 * Container for agents that exist in the world. Ticks the agents. Finds avatars for the agents.
 */
UCLASS()
class MLADAPTER_API UMLAdapterSession : public UObject
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAgentAvatarChangedDelegate, UMLAdapterAgent& /*Agent*/, AActor* /*OldAvatar*/);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBeginAgentRemove, UMLAdapterAgent& /*Agent*/);

	/** Get this session's cached world. */
	virtual UWorld* GetWorld() const override { return CachedWorld; }

	virtual void PostInitProperties() override;

	virtual void BeginDestroy() override;

	// @todo this needs further consideration.

	UGameInstance* GetGameInstance() const { return CachedWorld ? CachedWorld->GetGameInstance() : nullptr; }
	
	/** Sets the world that this session will use to find avatars for agents */
	virtual void SetWorld(UWorld* NewWorld);

	/** If there are agents waiting for an avatar, then check if a newly spawned actor fits the agent. */
	virtual void OnActorSpawned(AActor* InActor);

	virtual void OnPostWorldInit(UWorld& World);
	virtual void OnWorldCleanup(UWorld& World, bool bSessionEnded, bool bCleanupResources);
	virtual void OnGameModeInitialized(AGameModeBase& GameModeBase);
	virtual void OnGameModeMatchStateSet(FName InMatchState);
	virtual void OnGameModePostLogin(AGameModeBase* GameMode, APlayerController* NewPlayer);

	/** Mark the session as active. */
	virtual void Open();

	/** Mark the session as inactive and cleanup all agents. */
	virtual void Close();

	/** Call all agents' Sense(), Think(), and Act() methods. */
	virtual void Tick(float DeltaTime);

	/**
	 *	This is where Session can add Authority-side-specific functions by calling
	 *	UMLAdapterManager::Get().AddServerFunctionBind
	 */
	virtual void ConfigureAsServer();

	/**
	 *	This is where Session can add Client-side-specific functions by calling 
	 *	UMLAdapterManager::Get().AddClientFunctionBind. "Client" in this context means UnrealEngine game client, not
	 *	RPC client.
	 */
	virtual void ConfigureAsClient();

	/** Using FMLAdapter::InvalidAgentID for AgentID will reset all agents. */
	virtual void ResetWorld(FMLAdapter::FAgentID AgentID = FMLAdapter::InvalidAgentID);

	/** A session is done if the SimulationState == Finished or the game mode's match has ended. */
	bool IsDone() const;

	/** A session is ready if the SimulationState == InProgress or the game mode's match has started but not ended. */
	bool IsReady() const;

	/** Get the world time in seconds at the most recent time this session ticked. */
	float GetTimestamp() const { return LastTimestamp; }

	/** If true, the world's ticking will be controlled by the remote client. */
	void SetManualWorldTickEnabled(bool bEnable);

	/** Enable/disable the action durations with the specified time duration in seconds. */
	void EnableActionDuration(FMLAdapter::FAgentID AgentID, bool bEnable, float DurationSeconds);

	/** Resets the action duration flag if it has elapsed. Returns false if not reset yet or the agent is not found. Used with HasActionDurationElapsed by Manager. */
	bool TryResetActionDuration(FMLAdapter::FAgentID AgentID);

	FOnAgentAvatarChangedDelegate GetOnAgentAvatarChanged() { return OnAgentAvatarChanged; }
	FOnBeginAgentRemove GetOnBeginAgentRemove() { return OnBeginAgentRemove; }

	//----------------------------------------------------------------------//
	// Agent/Avatar management 
	//----------------------------------------------------------------------//

	/** Add a default agent as specified in the UMLAdapterSettings. */
	FMLAdapter::FAgentID AddAgent();

	/** Add an agent using the specified config. */
	FMLAdapter::FAgentID AddAgent(const FMLAdapterAgentConfig& InConfig);

	/**
	 *	@return Next valid agent ID. Note that the return value might be equal to ReferenceAgentID if there's only one
	 *	agent. Will be FMLAdapter::InvalidAgentId if no agents registered.
	 */
	FMLAdapter::FAgentID GetNextAgentID(FMLAdapter::FAgentID ReferenceAgentID) const;

	/** Returns the agent corresponding to the given ID. Returns nullptr if the ID is invalid. */
	UMLAdapterAgent* GetAgent(FMLAdapter::FAgentID AgentID);

	/** Remove the agent with the given ID from this session. */
	void RemoveAgent(FMLAdapter::FAgentID AgentID);

	/** Returns true if the agent with the given ID has a valid avatar assigned. */
	bool IsAgentReady(FMLAdapter::FAgentID AgentID) const;

	/** Finds avatar in given World for every avatar-less agent in AwaitingAvatar */
	void FindAvatars(UWorld& World);

	/**
	 *	Processes Agents and removes all agent avatars belonging to World. 
	 *	If World is null the function will remove all avatars.
	 */
	void RemoveAvatars(UWorld* World);

	/**
	 *	Finds a suitable avatar in InWorld (or CachedWorld, if InWorld is null) 
	 *	for given agent, as specified by FMLAdapterAgentConfig.AvatarClass
	 *	and confirmed by Agent->IsSuitableAvatar call. If no suitable avatar is 
	 *	found this agent will be added to "waiting list" (AwaitingAvatar)
	 *	@param bForceSearch if true will ignore whether the Agent is already waiting in 
	 *		AwaitingAvatar and will perform the search right away. Note that the Agent 
	 *		might still end up in AwaitingAvatar if there's no suitable avatars available 
	 *	@return True if an avatar has been assigned. False otherwise.*/
	virtual bool RequestAvatarForAgent(UMLAdapterAgent& Agent, UWorld* InWorld = nullptr, const bool bForceSearch = false);

	/** Calls RequestAvatarForAgent(UMLAdapterAgent& Agent, ...) after looking up the agent by its ID. */
	bool RequestAvatarForAgent(FMLAdapter::FAgentID& AgentID, UWorld* InWorld = nullptr);

	/** Sets a new avatar on the agent after clearing any existing avatar. */
	void BindAvatar(UMLAdapterAgent& Agent, AActor& Avatar);

	/** Remove the existing avatar from the given agent. */
	void ClearAvatar(UMLAdapterAgent& Agent);

	/** @todo this technically returns the count of all agents that ever existed given that we are not removing them yet. */
	int32 GetAgentsCount() const { return Agents.Num(); }

	/** Find an agent that is controlling the given avatar if one exists. Returns nullptr if not. */
	const UMLAdapterAgent* FindAgentByAvatar(AActor& Avatar) const;

	//----------------------------------------------------------------------//
	// debug 
	//----------------------------------------------------------------------//
#if WITH_GAMEPLAY_DEBUGGER
	void DescribeSelfToGameplayDebugger(class FGameplayDebuggerCategory& DebuggerCategory) const;
#endif // WITH_GAMEPLAY_DEBUGGER

protected:
	virtual void SetGameMode(AGameModeBase* GameModeBase);

	inline static uint32 HashAvatar(const AActor& Avatar)
	{
		return Avatar.GetUniqueID();
	}

	UPROPERTY()
	TObjectPtr<AGameModeBase> CachedGameMode;

	UPROPERTY()
	TObjectPtr<UWorld> CachedWorld;

	/** @see HashAvatar */
	UPROPERTY()
	TMap<uint32, TObjectPtr<UMLAdapterAgent>> AvatarToAgent;

	UPROPERTY()
	TArray<TObjectPtr<UMLAdapterAgent>> Agents;

	UPROPERTY()
	TArray<TObjectPtr<UMLAdapterAgent>> AwaitingAvatar;

	FOnAgentAvatarChangedDelegate OnAgentAvatarChanged;
	FOnBeginAgentRemove OnBeginAgentRemove;

	FDelegateHandle ActorSpawnedDelegateHandle;
	
	EMLAdapterSimState SimulationState;

	float LastTimestamp = -1.f;

	bool bActive = false;
	bool bTickWorldManually = false;

	mutable FCriticalSection AgentOpCS;

	struct FWorldTicker : public FTickableGameObject
	{
		TWeakObjectPtr<UWorld> CachedWorld;
		FWorldTicker(UWorld* InWorld) : CachedWorld(InWorld) {}
		virtual ~FWorldTicker();
		virtual void Tick(float DeltaTime) override;
		virtual UWorld* GetTickableGameObjectWorld() const override { return CachedWorld.Get(); }
		virtual TStatId GetStatId() const { return TStatId(); }
	};

	TSharedPtr<FWorldTicker> WorldTicker;
};
