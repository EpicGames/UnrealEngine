// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "Agents/MLAdapterAgent.h"
#include "AIController.h"
#include "GameFramework/Pawn.h"
#include "Sessions/MLAdapterSession.h"

#define LOCTEXT_NAMESPACE "AITestSuite_MLAdapterTest"

struct FMockAgentConfig_Actors : public FMLAdapterAgentConfig
{
	FMockAgentConfig_Actors()
	{
		bAutoRequestNewAvatarUponClearingPrev = false;
	}
};

struct FMockAgentConfig_OnlyActor : public FMLAdapterAgentConfig
{
	FMockAgentConfig_OnlyActor()
	{
		bAvatarClassExact = true;
		bAutoRequestNewAvatarUponClearingPrev = false;
	}
};

/** 
 *	this fixture is purposefully session-less where possible and a basic session instance is created to avoid
 *	certain edge cases. The tests using this fixture should not rely on session functionality 
 */
struct FMLAdapterTest_AvatarSettingFixture : public FAITestBase
{
	UMLAdapterAgent* Agent = nullptr;
	AActor* Actor = nullptr;
	APawn* Pawn = nullptr;
	AAIController* Controller = nullptr;

	virtual bool SetUp() override
	{
		UWorld& World = GetWorld();
		UMLAdapterSession* Session = NewObject<UMLAdapterSession>(&World);
		AITEST_NOT_NULL("Session", Session);
		Session->SetWorld(&World);
		const FMLAdapter::FAgentID AgentID = Session->AddAgent(FMockAgentConfig_Actors());
		Agent = Session->GetAgent(AgentID);
		Pawn = World.SpawnActor<APawn>();
		Controller = World.SpawnActor<AAIController>();
		Actor = World.SpawnActor<AActor>();

		return Agent && Actor && Pawn && Controller;
	}
};

IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMLAdapterTest_AvatarSettingFixture, "System.AI.MLAdapter.Agent", ActorAvatarSetting)
{
	//Initially avatar should not be null since there are matching actors
	Agent->Configure(FMockAgentConfig_OnlyActor());
	AITEST_NOT_NULL("Initially avatar", Agent->GetAvatar());
	AITEST_EQUAL("Initial avatar is of actor class", Agent->GetAvatar()->GetClass(), AActor::StaticClass());
	Agent->SetAvatar(Actor);
	AITEST_EQUAL("After assigning avatar should match the input actor", Agent->GetAvatar(), Actor);
	AITEST_NULL("Assigning a non-pawn avatar should not set agent\'s pawn", Agent->GetPawn());
	AITEST_NULL("Assigning a non-controller avatar should not set agent\'s controller", Agent->GetController());
	return true;
}

IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMLAdapterTest_AvatarSettingFixture, "System.AI.MLAdapter.Agent", ActorAvatarOverriding)
{
	Agent->SetAvatar(Actor);
	// overriding current avatar
	Agent->SetAvatar(Pawn);
	AITEST_NOT_NULL("After overriding avatar should not be null", Agent->GetAvatar());
	AITEST_EQUAL("After overriding avatar should match the input pawn", Agent->GetAvatar(), Pawn);
	return true;
}

IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMLAdapterTest_AvatarSettingFixture, "System.AI.MLAdapter.Agent", ActorAvatarClearing)
{
	Agent->SetAvatar(Actor);
	Agent->SetAvatar(nullptr);
	AITEST_NULL("Setting a null avatar should clear out agent\'s avatar", Agent->GetAvatar());
	return true;
}

IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMLAdapterTest_AvatarSettingFixture, "System.AI.MLAdapter.Agent", ActorAvatarDestruction)
{
	Agent->SetAvatar(Actor);
	Actor->Destroy();
	// the destruction of Actor avatar should get picked up by the Agent
	AITEST_NULL("Avatar after its destruction", Agent->GetAvatar());
	return true;
}

IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMLAdapterTest_AvatarSettingFixture, "System.AI.MLAdapter.Agent", PawnAvatarSetting)
{
	Agent->SetAvatar(Pawn);
	AITEST_EQUAL("After assigning avatar should match the input pawn", Agent->GetAvatar(), Pawn);
	AITEST_EQUAL("After assigning pawn-avatar should match the input pawn", Agent->GetPawn(), Pawn);
	AITEST_NULL("No controller since pawn has not been possessed yet", Agent->GetController());
	return true;
}

IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMLAdapterTest_AvatarSettingFixture, "System.AI.MLAdapter.Agent", ControllerAvatarSetting)
{
	Agent->SetAvatar(Controller);
	AITEST_EQUAL("After assigning avatar should match the input controller", Agent->GetAvatar(), Controller);
	AITEST_EQUAL("After assigning controller-avatar should match the input pawn", Agent->GetController(), Controller);
	AITEST_NULL("No pawn since the controller has not possessed a pawn yet", Agent->GetPawn());
	return true;
}

IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMLAdapterTest_AvatarSettingFixture, "System.AI.MLAdapter.Agent", PossesingWhileControllerAvatar)
{
	Agent->SetAvatar(Controller);
	AITEST_NULL("Setting pawn-less controller as agent should result in no pawn agent", Agent->GetPawn());
	Controller->Possess(Pawn);
	AITEST_EQUAL("Possessing a pawn by avatar-controller should make the pawn known to the controller", Agent->GetPawn(), Pawn);
	return true;
}

IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMLAdapterTest_AvatarSettingFixture, "System.AI.MLAdapter.Agent", AvatarOverriding)
{
	Agent->SetAvatar(Controller);
	Agent->SetAvatar(Actor);
	AITEST_EQUAL("Setting an avatar while one is already set should simply override the old avatar", Agent->GetAvatar(), Actor);
	AITEST_NULL("Pawn-avatar should get cleared", Agent->GetPawn());
	AITEST_NULL("Controller-avatar should get cleared", Agent->GetController());
	return true;
}

IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMLAdapterTest_AvatarSettingFixture, "System.AI.MLAdapter.Agent", BlockAssigningUnsuitableAvatar)
{
	Agent->Configure(FMockAgentConfig_OnlyActor());
	AITEST_NOT_NULL("Initial Avatar", Agent->GetAvatar());
	AActor* Avatar = Agent->GetAvatar();
	Agent->SetAvatar(Controller);
	AITEST_NOT_EQUAL("Not possible to assign controller avatar", Agent->GetAvatar(), Controller);
	AITEST_EQUAL("Still the original avatar", Agent->GetAvatar(), Avatar);
	
	return true;
}

#undef LOCTEXT_NAMESPACE
