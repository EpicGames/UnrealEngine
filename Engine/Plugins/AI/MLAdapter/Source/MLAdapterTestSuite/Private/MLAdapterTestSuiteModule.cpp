// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLAdapterTestSuiteModule.h"

#define LOCTEXT_NAMESPACE "MLAdapterTestSuite"

class FMLAdapterTestSuiteModule : public IMLAdapterTestSuiteModule
{
};

IMPLEMENT_MODULE(FMLAdapterTestSuiteModule, MLAdapterTestSuite)

#undef LOCTEXT_NAMESPACE
