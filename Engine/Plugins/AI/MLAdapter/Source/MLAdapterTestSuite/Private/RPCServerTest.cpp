// Copyright Epic Games, Inc. All Rights Reserved.

#include "RPCTestBase.h"
#if WITH_RPCLIB
#include "RPCWrapper/Server.h"
#include "RPCWrapper/rpclib_includes.h"

#define LOCTEXT_NAMESPACE "AITestSuite_MLAdapterTest"

void FRPCTestBase::TearDown() 
{
	UMLAdapterManager::Get().GetOnAddClientFunctions().Remove(BindClientHandle);
	UMLAdapterManager::Get().GetOnAddServerFunctions().Remove(BindServerHandle);
	UMLAdapterManager::Get().StopServer();

	delete RPCClient;
	RPCClient = nullptr;

	FAITestBase::TearDown();
}

//----------------------------------------------------------------------//
// TESTS 
//----------------------------------------------------------------------//

struct FRPCTest_StartStop : public FRPCTestBase
{
	virtual bool InstantTest() override
	{
		UMLAdapterManager::Get().StartServer(DefaultServerPort, EMLAdapterServerMode::Client);
		AITEST_TRUE("Is server running", UMLAdapterManager::Get().IsRunning());
		UMLAdapterManager::Get().StopServer();
		AITEST_FALSE("Is server stopped", UMLAdapterManager::Get().IsRunning());
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRPCTest_StartStop, "System.AI.MLAdapter.RPC.ServerStartStop")

struct FRPCTest_BasicBinds : public FRPCTestBase
{
	uint8 bClientFooCalled : 1;
	uint8 bServerFooCalled : 1;
	uint8 CallCount : 6;
	EMLAdapterServerMode ServerMode = EMLAdapterServerMode::Client;
	  
	FRPCTest_BasicBinds() : bClientFooCalled(false), bServerFooCalled(false)
	{}

	virtual bool SetUp() override
	{
		UMLAdapterManager::Get().StartServer(DefaultServerPort, ServerMode);
		RPCClient = new rpc::client("127.0.0.1", DefaultServerPort);
		return RPCClient != nullptr;
	}

	// wait for any of the functions to get called checking CallCount
	// virtual bool Update() override

	virtual void SetUpClientBinds(FRPCServer& Server) override
	{
		Server.bind("client_foo", [this]()
		{
			bClientFooCalled = true; 
			++CallCount;
		});
	}
	virtual void SetUpServerBinds(FRPCServer& Server) override
	{
		Server.bind("server_foo", [this]()
		{ 
			bServerFooCalled = true; 
			++CallCount;
		});
	}
};

struct FRPCTest_ClientBinds : public FRPCTest_BasicBinds
{
	virtual bool SetUp() override
	{
		bool bSuccess = false;
		FRPCTest_BasicBinds::SetUp();		
		// ordering this way to make sure we first call the function that's not 
		// likely to throw an exception. RPC client will throw one if function of 
		// given name is not found 
		try
		{
			RPCClient->call("client_foo");
			RPCClient->call("server_foo");
		}
		catch (...)
		{
			// this is expected if we call a function that has not been bound
			bSuccess = true;
		}
		return bSuccess;
	}
	virtual bool InstantTest() override
	{
		AITEST_TRUE("Only one function should get called", CallCount == 1);
		AITEST_TRUE("Only the client function should get called", bClientFooCalled);
		AITEST_FALSE("The server function should not get called", bServerFooCalled);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRPCTest_ClientBinds, "System.AI.MLAdapter.RPC.ClientBinds")

struct FRPCTest_ServerBinds : public FRPCTest_BasicBinds
{
	virtual bool SetUp() override
	{
		bool bSuccess = false;
		ServerMode = EMLAdapterServerMode::Server;
		FRPCTest_BasicBinds::SetUp();
		// ordering this way to make sure we first call the function that's not 
		// likely to throw an exception. RPC client will throw one if function of 
		// given name is not found 
		try
		{
			RPCClient->call("server_foo");
			RPCClient->call("client_foo");
		}
		catch (...)
		{
			// this is expected if we call a function that has not been bound
			bSuccess = true;
		}
		return bSuccess;
	}

	virtual bool InstantTest() override
	{
		AITEST_TRUE("Only one function should get called", CallCount == 1);
		AITEST_TRUE("Only the server function should get called", bServerFooCalled);
		AITEST_FALSE("The client function should not get called", bClientFooCalled);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRPCTest_ServerBinds, "System.AI.MLAdapter.RPC.ServerBinds")

#undef LOCTEXT_NAMESPACE

#endif // WITH_RPCLIB
