// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "AITestsCommon.h"
#include "Misc/App.h"
#include "Managers/MLAdapterManager.h"


#if WITH_RPCLIB
namespace rpc
{
	class client;
}

struct FRPCTestBase : public FAITestBase
{
	enum 
	{
		DefaultServerPort = 10101
	}; 

	EMLAdapterServerMode Mode = EMLAdapterServerMode::Client;
	FDelegateHandle BindClientHandle;
	FDelegateHandle BindServerHandle;
	rpc::client* RPCClient = nullptr;

	FRPCTestBase()
	{
		BindClientHandle = UMLAdapterManager::Get().GetOnAddClientFunctions().AddLambda([this](FRPCServer& Server)
		{
			SetUpClientBinds(Server);
		});
		BindServerHandle = UMLAdapterManager::Get().GetOnAddServerFunctions().AddLambda([this](FRPCServer& Server)
		{
			SetUpServerBinds(Server);
		});
	}

	virtual void SetUpClientBinds(FRPCServer& Server) {}
	virtual void SetUpServerBinds(FRPCServer& Server) {}
	
	virtual void TearDown() override;
};
#endif // WITH_RPCLIB
