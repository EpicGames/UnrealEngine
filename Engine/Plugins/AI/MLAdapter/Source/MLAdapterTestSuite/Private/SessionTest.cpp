// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "Engine/GameInstance.h"
#include "Managers/MLAdapterManager.h"
#include "Sessions/MLAdapterSession.h"
#include "Agents/MLAdapterAgent.h"
#include "AIController.h"
#include "GameFramework/Pawn.h"
#include "Engine/Engine.h"


#define LOCTEXT_NAMESPACE "AITestSuite_MLAdapterTest"

/**
 *	this fixture creates a game instance and can react to changes done to pawn.controller
 *	also, the session instance is created via the UMLAdapterManager so all other notifies should get through as well, most 
 *	notably world-related ones
 */
struct FMLAdapterTest_WithSession : public FAITestBase
{
	UMLAdapterAgent* Agent = nullptr;
	AActor* Actor = nullptr;
	APawn* Pawn = nullptr;
	AAIController* Controller = nullptr;
	UGameInstance* GameInstance = nullptr;
	FMLAdapter::FAgentID AgentID = FMLAdapter::InvalidAgentID;
	UMLAdapterSession* Session = nullptr;

	virtual UWorld& GetWorld() const override
	{
		return GameInstance && GameInstance->GetWorld()
			? *GameInstance->GetWorld()
			: FAITestBase::GetWorld();
	}

	virtual bool SetUp() override
	{
		Session = &UMLAdapterManager::Get().GetSession();

		GameInstance = NewObject<UGameInstance>(GEngine);
		AITEST_NOT_NULL("GameInstance", GameInstance);
		GameInstance->InitializeStandalone();
		
		FMLAdapterAgentConfig EmptyConfig;
		AgentID = Session->AddAgent(EmptyConfig);
		Agent = Session->GetAgent(AgentID);

		UWorld& World = GetWorld();
		Actor = World.SpawnActor<AActor>();
		Pawn = World.SpawnActor<APawn>();
		Controller = World.SpawnActor<AAIController>();

		return Agent && Actor && Pawn && Controller;
	}

	virtual void TearDown() override
	{
		if (Session)
		{
			UMLAdapterManager::Get().CloseSession(*Session);
		}
		FAITestBase::TearDown();
	}
};

IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMLAdapterTest_WithSession, "System.AI.MLAdapter.Agent", PossessingWhilePawnAvatar)
{
	Agent->SetAvatar(Pawn);
	AITEST_NULL("Setting unpossessed pawn as avatar results in no controller", Agent->GetController());
	Controller->Possess(Pawn);
	AITEST_EQUAL("After possessing the pawn the controller should be known to the agent", Agent->GetController(), Controller);
	return true;
}

IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMLAdapterTest_WithSession, "System.AI.MLAdapter.Agent", SessionAssigningAvatar)
{
	FMLAdapterAgentConfig NewConfig;
	NewConfig.AvatarClassName = APawn::StaticClass()->GetFName();
	Agent->Configure(NewConfig);
	AITEST_EQUAL("Calling configure should make the session instance pick a pawn avatar for the agent", Agent->GetPawn(), Pawn);
	return true;
}

IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMLAdapterTest_WithSession, "System.AI.MLAdapter.Agent", ChangingAvatarClassOnTheFly)
{
	FMLAdapterAgentConfig NewConfig;
	NewConfig.AvatarClassName = APawn::StaticClass()->GetFName();
	Agent->Configure(NewConfig);
	AITEST_EQUAL("Calling configure should make the session instance pick a pawn avatar for the agent", Agent->GetPawn(), Pawn);
	return true;
}

IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMLAdapterTest_WithSession, "System.AI.MLAdapter.Agent", FindingNewPawnAfterDeath)
{
	FMLAdapterAgentConfig NewConfig;
	NewConfig.AvatarClassName = APawn::StaticClass()->GetFName();
	Agent->Configure(NewConfig);

	AITEST_NOT_NULL("Session", Session);

	APawn* Pawn2 = Session->GetWorld()->SpawnActor<APawn>(FVector::ZeroVector, FRotator::ZeroRotator);
	Pawn->Destroy();
	// Avatar auto-selection should pick Pawn2 after Pawn is destroyed
	AITEST_EQUAL("Auto-picked avatar and the other pawn", Agent->GetAvatar(), Pawn2);

	return true;
}

IMPLEMENT_INSTANT_TEST_WITH_FIXTURE(FMLAdapterTest_WithSession, "System.AI.MLAdapter.Agent", UnPossesingWhileControllerAvatar)
{
	Agent->SetAvatar(Controller);
	Controller->Possess(Pawn);
	Controller->UnPossess();
	AITEST_NULL("After the controller unpossessing its pawn the agent should automatically update", Agent->GetPawn());
	return true;
}

#undef LOCTEXT_NAMESPACE
