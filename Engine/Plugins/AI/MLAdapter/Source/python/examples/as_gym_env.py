# Copyright Epic Games, Inc. All Rights Reserved.

import gym
import unreal.mladapter.logger as logger
from unreal.mladapter.utils import random_action
import unreal.mladapter.utils


logger.set_level(logger.DEBUG)

# see unreal.mladapter.utils.ArgumentParser.__init__ for list of default parameters
parser = unreal.mladapter.utils.ArgumentParser()
parser.add_argument('--iter', type=int, default=3, help='number of games to play')
args = parser.parse_args()

env = gym.make(args.env, server_port=args.port)

for i in range(args.iter): 
    env.reset()
    reward = 0
    done = False
    while not env.game_over:
        _, reward, done, _ = env.step(random_action(env))

    print('{}: Score: {}'.format(i, reward))

env.close()