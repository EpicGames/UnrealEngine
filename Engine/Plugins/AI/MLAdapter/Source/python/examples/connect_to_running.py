# Copyright Epic Games, Inc. All Rights Reserved.

from unreal.mladapter.envs import *
from unreal.mladapter.utils import random_action, DEFAULT_PORT
import unreal.mladapter.logger as logger
import argparse

"""
This script will attempt to connect to a running instance of UnrealEngine using DEFAULT_PORT assuming that the 
instance's unreal.mladapter rpc server is listening at that specific port.
Closing this script won't close the running UnrealEngine instance, but should re-enable the runtime mode on it.
"""

# environment wrapper we want to use. 
env_cls = ActionRPG

# setting logging level so that we see all there is to see
logger.set_level(logger.DEBUG)

# note that we're using stock ArgumentParser here not the unreal.mladapter.utils.ArgumentParser
parser = argparse.ArgumentParser()
parser.add_argument('--port', type=int, default=DEFAULT_PORT)
args = parser.parse_args()

# using ue_params=None means "attempt to connect to a running instance"
env = env_cls(ue_params=None, server_port=args.port)

game_over = False
env.reset()
# using local variable game_over to demonstrate a concept. This is exactly the same as using env.game_over 
while not game_over:
    # UnrealEnv.step returns a tuple (observations, reward, is_game_over, additional_info)
    # we just care about 'game_over' here, ignoring the rest
    _, _, game_over, _ = env.step(random_action(env))

print('Done')
