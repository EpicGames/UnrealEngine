# Copyright Epic Games, Inc. All Rights Reserved.

from unreal.mladapter.envs import ActionRPG
from unreal.mladapter.utils import random_action, DEFAULT_PORT
import unreal.mladapter.logger as logger
from unreal.mladapter.runner import UEParams, set_debug 
import argparse

"""
This script shows how do manually config an environment
"""

env_cls = ActionRPG

# affects how much logging we'll see
logger.set_level(logger.DEBUG)

# set depending on whether you want to use DevEditor or DebugEditor
set_debug(True)

# note that we're using stock ArgumentParser here not the unreal.mladapter.utils.ArgumentParser
parser = argparse.ArgumentParser()
parser.add_argument('--nothreads', type=int, default=0)
parser.add_argument('--norendering', type=int, default=0)
parser.add_argument('--nosound', type=int, default=0)
parser.add_argument('--resx', type=int, default=800)
parser.add_argument('--resy', type=int, default=600)
parser.add_argument('--port', type=int, default=DEFAULT_PORT)
args = parser.parse_args()

# UEParams configures an UnrealEngine instance we want to launch. If you want to connect to an UnrealEngine instance 
# that's already running just use 'params = None'
params = UEParams()
params.rendering(args.norendering == 0)
params.single_thread(args.nothreads == 1)
params.sound(args.nosound == 0)
params.resolution(args.resx, args.resy)

# call params.add_option or params.add_param to supply additional parameters

# instead of above we suggest using the following:
# parser = unreal.mladapter.utils.ArgumentParser()
# args = parser.parse_args()
# params = UEParams.from_args(args)


# create environment wrapper. If params != None then UnrealEngine instance will get launched. When launching an instance
# then env_class._project_name controls which project gets launched 
env = env_cls(ue_params=params, server_port=args.port)

# standard gym loop
env.reset()
while not env.game_over:
    env.step(random_action(env))

# in this specific context this call is redundant since it will get auto-called as part of script's teardown
env.close()
print('Done')
