# Copyright Epic Games, Inc. All Rights Reserved.

from setuptools import setup, find_packages
import sys
import os.path

# Don't import unreal.mladapter module here, since dependencies may not be installed
sys.path.insert(0, os.path.join(os.path.dirname(__file__), 'unreal', 'mladapter'))
from version import VERSION

module_name = 'unreal.mladapter'
module_root = os.path.dirname(__file__)

sys.path.insert(0, os.path.join(module_root, module_name))

packages = [package for package in find_packages(module_root) if module_name in package]

setup(name='unreal.mladapter',
      version=VERSION,
      description='Extension to OpenAI Gym adding capability to interface with UnrealEngine projects using '
                  'unreal.mladapter plugin.',
      author='Mieszko Zielinski @ Epic Games',
      author_email='mieszko.zielinski@epicgames.com',
      license='',
      packages=packages,
      zip_safe=True,
      install_requires=['gym', 'msgpack-rpc-python', 'numpy'],
      python_requires='>=3.5.*',
)