# Copyright Epic Games, Inc. All Rights Reserved.

import msgpackrpc
from unreal.mladapter.client import Client
import unreal.mladapter.utils as utils


class MockFunctions(object):
    def __init__(self):
        # making sure the class has all the required functions
        self.__dict__[Client.FUNCNAME_LIST_FUNCTIONS] = self._list_functions
        self.__dict__[Client.FUNCNAME_PING] = self._ping

    def sum(self, x, y):
        return x + y

    def foo(self):
        return True

    def _ping(self):
        return True

    def _list_functions(self):
        return [Client.FUNCNAME_LIST_FUNCTIONS, Client.FUNCNAME_PING, 'foo', 'sum', 'non_implemented']


class MockServer(msgpackrpc.Server):
    def __init__(self, port, dispatcher=MockFunctions()):
        super(MockServer, self).__init__(dispatcher, pack_encoding=None)
        self.listen(msgpackrpc.Address(utils.LOCALHOST, port))
        # not calling self.start() since it's a blocking call. Triggering a thread instead
        import threading
        self._thread = threading.Thread(target=self.start)  # , args=(1,))
        self._thread.start()

    def close(self):
        # del self._thread
        self.stop()
        self._thread.join(timeout=0.01)

    def __enter__(self):
        return self

    def __exit__(self, exc_type, exc_val, exc_tb):
        # del self._thread
        self.stop()
        self._thread.join(timeout=0.01)