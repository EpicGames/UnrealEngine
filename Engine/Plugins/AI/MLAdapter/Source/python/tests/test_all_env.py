# Copyright Epic Games, Inc. All Rights Reserved.

# Note that this is not a unit-testing suite, just a script one can run to get all the registered UnrealEngine
# environments launched, go through random game and close.

import gym
import unreal.mladapter.logger as logger
from unreal.mladapter.utils import random_action
from threading import Thread

WITH_THREADING = True
logger.set_level(logger.DEBUG)
ue_envs = [e.id for e in gym.envs.registry.all() if 'UnrealEngine' in e.id]


def launch_env(env_name):
    try:
        env = gym.make(env_name, server_port=None, timeout=10)
    except gym.error.Error:
        print('{}: FAILED TO LAUNCH'.format(env_name))
        return
    env.reset()
    while not env.game_over:
        env.step(random_action(env))    
    print('{}: done'.format(env_name))
    env.close()
    

if __name__ == '__main__':
    threads = []
    for name in ue_envs:
        if WITH_THREADING:
            t = Thread(target=launch_env, args=(name,))
            t.start()
            threads.append((name, t))
        else:
            launch_env(name)
            
    for name, t in threads:
        print('Joining {}'.format(name), end='')
        t.join()
        print(' DONE.'.format(name))
