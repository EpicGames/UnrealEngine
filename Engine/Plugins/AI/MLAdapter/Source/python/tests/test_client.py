# Copyright Epic Games, Inc. All Rights Reserved.
 
import unittest
from unreal.mladapter.client import Client
import unreal.mladapter.utils as utils
from unreal.mladapter.error import *
from mock_server import MockServer
from time import time


class SocketTest(unittest.TestCase):
    def test_free_socket(self):
        port = utils.find_available_port()
        self.assertTrue(port > 0)
        self.assertTrue(utils.is_port_available(port))
        
        
class ClientTest(unittest.TestCase):
    def test_connection_timeout(self):
        c = Client(timeout=0.01, server_port=None)
        with self.assertRaises(ConnectionTimeoutError):
            c.call('foo')

    def test_reconnection_limit(self):
        timeout = 1024
        c = Client(timeout=timeout, server_port=None, reconnect_limit=1)
        time_start = time()
        with self.assertRaises(ReconnectionLimitReached):
            c.ensure_connection()
        self.assertLess(time() - time_start, timeout)

    def test_reconnect_to_new(self):
        c = Client(server_port=None, reconnect_limit=100)
        import concurrent.futures        
        with concurrent.futures.ThreadPoolExecutor() as executor:
            # calling rpc function before the server is launched. The call should just hang in there waiting for the 
            # server to come on line (up until set limit of reconnect attempts)
            future = executor.submit(c.call, 'sum', 1, 2)
            with MockServer(c.address.port) as s:                
                return_value = future.result()
                self.assertEqual(return_value, 3)
            
    def test_rcp_functions(self):
        c = Client()
        with MockServer(c.address.port) as s:            
            self.assertTrue(c.call('foo'))
            with self.assertRaises(RPCError):
                c.call('not_existing')                
            
            c.add_functions()
            self.assertTrue(c.call(Client.FUNCNAME_PING))
            self.assertTrue(c.call('sum', 1, 2) == c.sum(1, 2))


if __name__ == '__main__':
    unittest.main()
