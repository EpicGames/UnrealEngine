# Copyright Epic Games, Inc. All Rights Reserved.
 
import unittest
from unreal.mladapter.core import UnrealEnv, AgentConfig 
from unreal.mladapter.error import *
import json
import string
import random


def equal(a, b):
    if '__eq__' in a.__class__.__dict__:
        return a == b
    
    if a.__class__ != b.__class__:
        return False
    
    for key in a.__dict__:
        if a.__dict__[key] != b.__dict__[key]:
            return False
    return True


def random_string(length):
    letters = string.ascii_letters
    return ''.join(random.choice(letters) for _ in range(length))
    

class AgentConfigTest(unittest.TestCase):    
    def test_creation_from_json(self):
        from_json = AgentConfig.from_json('{}')
        default = AgentConfig()
        self.assertTrue(equal(from_json, default))
        self.assertEqual(default.as_json, from_json.as_json)
        
        with self.assertRaises(json.JSONDecodeError):
            AgentConfig.from_json('{')

    def test_configuring(self):
        c = AgentConfig()
        with self.assertRaises(ValueError):
            c.add_sensor('sensor1', 'param1')

        self.assertTrue(len(c.sensors) == 0)
        c.add_sensor('sensor1', {'param1': 'value1'})
        self.assertTrue(len(c.sensors) == 1)
        self.assertEqual(c.sensors['sensor1']['params'], {'param1': 'value1'})
        c.add_sensor('sensor1', {'param2': 'value2'})
        self.assertTrue(len(c.sensors) == 1)
        self.assertEqual(c.sensors['sensor1']['params'], {'param2': 'value2'})
        with self.assertRaises(KeyError):
            tmp = c.sensors['sensor1']['params']['param1']

        self.assertTrue(len(c.actuators) == 0)
        c.add_actuator('actuator1', {'param1': 'value1'})
        self.assertTrue(len(c.actuators) == 1)
        self.assertEqual(c.actuators['actuator1']['params'], {'param1': 'value1'})
        c.add_actuator('actuator1', {'param2': 'value2'})
        self.assertTrue(len(c.actuators) == 1)
        self.assertEqual(c.actuators['actuator1']['params'], {'param2': 'value2'})
        with self.assertRaises(KeyError):
            tmp = c.actuators['actuator1']['params']['param1']        
            
    def test_export_to_json(self):
        labels = [random_string(10) for _ in range(6)]
        c = AgentConfig()
        c.avatarClassName = 'Foo'
        c.agentClassName = 'Bar'
        c.add_sensor(labels[0], {labels[1]: labels[2]})
        c.add_actuator(labels[3], {labels[4]: labels[5]})
        as_json = c.as_json
        self.assertTrue(as_json)
        self.assertIn('Foo', as_json)
        self.assertIn('Bar', as_json)
        c.avatarClassName = ''
        as_json = c.as_json
        self.assertNotIn('Foo', as_json)
        self.assertIn('Bar', as_json)
        for s in labels:
            self.assertIn(s, as_json)

        d = AgentConfig.from_json(as_json)
        self.assertTrue(equal(c, d))
        
        
class EnvConnectionTest(unittest.TestCase):
    def test_no_auto_connect(self):
        env = UnrealEnv(auto_connect=False)
        self.assertFalse(env.is_connected())
        with self.assertRaises(NotConnected):
            env.reset()
        env.close()
        self.assertIsNone(env.conn)


if __name__ == '__main__':
    unittest.main()
