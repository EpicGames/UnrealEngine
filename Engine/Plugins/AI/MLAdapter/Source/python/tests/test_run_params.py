# Copyright Epic Games, Inc. All Rights Reserved.
 
import unittest
import unreal.mladapter.runner as runner
from unreal.mladapter.runner import UEParams, UERunner
from unreal.mladapter.error import *
import unreal.mladapter.logger as logger


class RunnerUtilsTest(unittest.TestCase):
    def test_environment_variables(self):
        import os
        self.assertIsNotNone(runner.UEDEVBINKEY)
        self.assertNotEqual(runner.UEDEVBINKEY, '')
        self.assertIn(runner.UEDEVBINKEY, os.environ)
        self.assertTrue(os.path.isdir(os.environ[runner.UEDEVBINKEY]))

    def test_exec_path(self):
        import os
        bin_path = os.environ[runner.UEDEVBINKEY]
        self.assertIn(bin_path, runner._get_exec_path())
        runner.set_executable('')
        self.assertNotIn(bin_path, runner._get_exec_path())
        runner.set_executable('foobar')
        self.assertIn('foobar', runner._get_exec_path())
        runner.set_executable(None)
        self.assertNotIn('foobar', runner._get_exec_path())
        self.assertIn(bin_path, runner._get_exec_path())


class ParamsTest(unittest.TestCase):
    def test_creation(self):
        params = UEParams.from_args(None)
        self.assertIsNotNone(params)

    def test_setting_default_map(self):
        old_map_name = 'foo'
        new_map_name = 'bar'
        
        params = UEParams.from_args(None)
        self.assertEqual(params.map_name, '')
        params.set_default_map_name(new_map_name)
        self.assertEqual(params.map_name, new_map_name)
        
        params = UEParams(map_name=old_map_name)
        self.assertEqual(params.map_name, old_map_name)
        params.set_default_map_name(new_map_name)
        self.assertEqual(params.map_name, new_map_name)
        
    def test_options(self):
        option_name = 'foobar'
        
        # adding an option
        params = UEParams()
        self.assertNotIn(option_name, params.as_command_line())
        params.add_option(option_name)
        self.assertIn(option_name, params.as_command_line())
        
        # alternative way
        params2 = UEParams()
        params2.set_enable_option(option_name, True)
        self.assertEqual(params.as_command_line(), params2.as_command_line())

        # removing an option
        params.set_enable_option(option_name, False)
        self.assertNotIn(option_name, params.as_command_line())
        
        for _ in range(10):
            params.set_enable_option(option_name, True)
        self.assertIn(option_name, params.as_command_line())
        params.set_enable_option(option_name, False)
        self.assertNotIn(option_name, params.as_command_line())
        
        # test adding a list of options
        options = [option_name+str(i) for i in range(10)]
        params = UEParams()
        params.add_option(options)
        cmd = params.as_command_line()
        for o in options:
            self.assertIn(o, cmd)
        
    def test_params(self):
        param_name = 'foobar'
        param_value = 'baz'
        param_value2 = 'qux'
        
        # adding a parameter
        params = UEParams()
        self.assertNotIn(param_name, params.as_command_line())
        self.assertNotIn(param_value, params.as_command_line())    
        params.add_param(param_name, param_value)
        self.assertIn(param_name, params.as_command_line())
        self.assertIn(param_value, params.as_command_line())
        
        # changing param name
        params.add_param(param_name, param_value2)
        self.assertIn(param_name, params.as_command_line())
        self.assertIn(param_value2, params.as_command_line())
        self.assertNotIn(param_value, params.as_command_line())
        
        # removing a parameter
        params.remove_param(param_name)
        self.assertNotIn(param_name, params.as_command_line())
        self.assertNotIn(param_value2, params.as_command_line())
        self.assertNotIn(param_value, params.as_command_line())

    def test_command_line_construction(self):
        map_name = 'foobar'
        custom_value = 'bazqux'
        params = UEParams(map_name=map_name, custom=custom_value)
        cmd = params.as_command_line()
        self.assertIsNotNone(cmd)
        self.assertTrue(len(cmd) > 0)
        self.assertIn(map_name, cmd)
        self.assertIn(custom_value, cmd)


class RunnerTest(unittest.TestCase):
    def test_running(self):
        logging_level = logger.get_level()
        
        logger.set_level(logger.DISABLED)
        runner.set_executable('')
        with self.assertRaises(FailedToLaunch):
            UERunner.run(None, UEParams(), None)
        runner.set_executable(None)
        
        logger.set_level(logging_level)


if __name__ == '__main__':
    unittest.main()
