# Copyright Epic Games, Inc. All Rights Reserved.

import unittest
import unreal.mladapter.utils as utils
import unreal.mladapter.spaces as spaces
from gym.spaces import Box, Discrete, MultiDiscrete, MultiBinary, Tuple, Dict
import json

simple_box4 = Box(-1, 1, shape=(4,))
simple_box2 = Box(-1, 1, shape=(2,))

json_samples = [(b'{"0":"{\\"Discrete\\":10}"}', Discrete(10)),
                (b'{"b":"{\\"MultiDiscrete\\":11}"}', MultiDiscrete(11)),
                (b'{"1":"{\\"Box\\":[-1.000000,2.5,5]}"}', Box(-1, 2.5, shape=(5,))),
                (b'{"4":"{\\"Tuple\\":[{\\"Box\\":[-1.000000,1.000000,4]},{\\"Box\\":[-2.000000,3.000000,8]}]}"}',
                 Tuple((simple_box4, Box(-2, 3, shape=(8,))))),
                (b'{}', None),
                (b'{"AIPerception":"{\\"Tuple\\":[{\\"Box\\":[-1.000000,1.000000,4]},{\\"Box\\":[-1.000000,1.000000,4]}'
                 b',{\\"Box\\":[-1.000000,1.000000,4]}]}","Attribute":"{\\"Box\\":[-1.000000,1.000000,2]}"}',
                 Tuple((Tuple((simple_box4, simple_box4, simple_box4)), simple_box2)))]


class SpaceBuildingTest(unittest.TestCase):
    def test_invalid_json(self): 
        self.assertIsNone(spaces.gym_space_from_mladapter(''))
        self.assertIsNone(spaces.gym_space_from_mladapter(None))
        with self.assertRaises(json.JSONDecodeError):
            spaces.gym_space_from_mladapter('notjson')
            
    def test_invalid_parameters(self):
        with self.assertRaises(TypeError):
            spaces.gym_space_from_list({'0': '0'})
        with self.assertRaises(TypeError):
            spaces.gym_space_from_list(1)
        
    def test_space_creation(self):
        space_samples = [(('Discrete', 12), Discrete(12)),
                         (('MultiDiscrete', 64), MultiDiscrete(64)),
                         (('Box', (-2, 2, 7)), Box(-2, 2, shape=(7,)))]
        
        for sample in space_samples:
            self.assertEqual(spaces.create_space(*sample[0]), sample[1])
                            
        for sample in json_samples:    
            space = spaces.gym_space_from_mladapter(sample[0])
            self.assertEqual(sample[1], space)


if __name__ == '__main__':
    unittest.main()
