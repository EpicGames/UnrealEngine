# Copyright Epic Games, Inc. All Rights Reserved.
 
import msgpackrpc
from msgpackrpc.error import TransportError
from .error import ReconnectionLimitReached
from . import logger
from .utils import *
import threading


class Client(msgpackrpc.Client):
    # crucial function names
    FUNCNAME_LIST_FUNCTIONS = 'list_functions'
    FUNCNAME_PING = 'ping'
    
    __port_range_start = DEFAULT_PORT
    __port_range_span = 128
    __next_available_port_offset = 0
    __lock = threading.Lock()
    
    def __init__(self, server_address=LOCALHOST, server_port=DEFAULT_PORT, timeout=DEFAULT_TIMEOUT, reconnect_limit=1024, **kwargs):
        
        if server_port is None:
            server_port = find_available_port(server_address)
        
        address = msgpackrpc.Address(server_address, server_port)
        # using pack_encoding=None since it's using encodings is deprecated (via msgpack.Packer)
        super().__init__(address, timeout=timeout, pack_encoding=None, reconnect_limit=reconnect_limit, **kwargs)
        #self._restart = lambda: self.__init__(server_address, server_port, **kwargs)

    def ensure_connection(self):
        logger.info('attempting connection at at port {}:{}'.format(self.address.host, self.address.port))
        try:
            self.call(Client.FUNCNAME_PING)
        except TransportError as e:
            # a bit hacky, but if underlying CODEs change unit tests will catch it
            if e.CODE == 'Retry connection over the limit':
                raise ReconnectionLimitReached
            raise   # else

    def _add_function(self, function_name):
        self.__dict__[function_name] = lambda *args: self.call(function_name, *args)

    def add_functions(self):
        self.ensure_connection()
        function_list = self.call(Client.FUNCNAME_LIST_FUNCTIONS)
        for fname in map(lambda x: x.decode('utf-8'), function_list):
            self._add_function(fname)            
        logger.debug('Functions bound: {}'.format(function_list))

    @classmethod
    def connect(cls):
        """ Returns an instance of the default client"""
        return cls()

    @property
    def connected(self):
        # breach of encapsulation but there's no other way to access this information
        return self._transport._connecting > 0 and not self._transport._closed
