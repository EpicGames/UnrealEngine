# Copyright Epic Games, Inc. All Rights Reserved.

import msgpackrpc
import time
import json
import gym
import gym.spaces
from . import logger
from . import spaces
from .client import Client
from .runner import UERunner
from .utils import LOCALHOST, DEFAULT_PORT, DEFAULT_TIMEOUT
from .error import FailedToLaunch, NotConnected
from .error import ReconnectionLimitReached, UnrealEngineInstanceNoLongerRunning, ConnectionTimeoutError, UnableToReachRPCServer

INVALID_ID = 4294967295  # uint32(-1)


class AgentConfig:
    
    def __init__(self, config={}):
        self.sensors = {}
        self.actuators = {}
        self.avatarClassName = 'PlayerController'
        self.agentClassName = ''
        self.bAutoRequestNewAvatarUponClearingPrev = True
        self.bAvatarClassExact = False

        if config:
            self.__dict__ = config

    def add_sensor(self, sensor_name, sensor_params={}):
        if type(sensor_params) != dict:
            raise ValueError('sensor_params must be a dict')
        self.sensors[sensor_name] = {'params': sensor_params}

    def add_actuator(self, actuator_name, actuator_params={}):
        if type(actuator_params) != dict:
            raise ValueError('sensor_params must be a dict')
        self.actuators[actuator_name] = {'params': actuator_params}

    @property
    def as_json(self):
        return json.dumps(self, default=lambda x: x.__dict__)

    @classmethod
    def from_json(self, json_string):
        if type(json_string) == bytes:
            json_string = json_string.decode('utf-8')
        return AgentConfig(json.loads(json_string))
    
    
class UnrealEnv(gym.Env):
    PROJECT_NAME = 'GenericUnrealEnvironment'
    
    def __init__(self, server_address=LOCALHOST, server_port=DEFAULT_PORT, agent_config=None, reacquire=True, 
                 realtime=False, auto_connect=True, timeout=DEFAULT_TIMEOUT, ue_params=None, project_name=None,
                 use_preconfigured_agent=False, action_duration_seconds=None):

        # It's expected that the user passes in project_name or implements a wrapper defining PROJECT_NAME
        self._project_name = project_name or self.__class__.PROJECT_NAME

        if not use_preconfigured_agent and agent_config is None:
            agent_config = self.__class__.default_agent_config()

        self.__agent_config = agent_config
        self.__rpc_client = Client(server_address, server_port, timeout)
        server_port = self.__rpc_client.address.port
        self._debug_id = server_port
        self.__server_port = server_port
        self._realtime = None
        self._action_duration_seconds = action_duration_seconds

        self.__engine_process = None
        if ue_params is not None:
            try:
                self.__engine_process = UERunner.run(self._project_name, ue_params, server_port)
            except FailedToLaunch:
                # failed to launch the engine
                self.__rpc_client.close()
                self.__rpc_client = None
                raise

        self._frames_step = 1
        # stores number of actions performed and frames skipped. Not counting skipped time in realtime mode
        self._steps_performed = 0
        self.name = 'Not connected yet'

        self._set_realtime(realtime)
        if auto_connect:
            self.connect(reacquire)
            
    def connect(self, reacquire=True):
        # if this takes too long it's possible there's a dangling server instance that failed to shutdown (i.e. it's
        # not the one you're trying to connect). Try using a different port.
        try:
            self.__rpc_client.ensure_connection()
        except (ReconnectionLimitReached, ConnectionTimeoutError):
            if self.__engine_process and self.__engine_process.poll() is not None:
                raise UnrealEngineInstanceNoLongerRunning
            raise UnableToReachRPCServer
        
        self.__rpc_client.add_functions()
        self.name = self.__rpc_client.get_name().decode('utf-8')
        logger.info('connected to {} at port {}'.format(self.name, self.__server_port))

        self._setup_agents(reacquire)
        self._set_realtime(self._realtime)
        
    def is_connected(self):
        return self.__rpc_client and self.__rpc_client.connected

    def _setup_agents(self, reacquire):
        # if there's already an agent created in the environment
        self.__agent_id = self.__rpc_client.get_recent_agent() if reacquire else INVALID_ID
        if self.__agent_id == INVALID_ID:
            self.__agent_id = self._new_agent(self.__agent_config)
        elif self.__agent_config:
            self.__rpc_client.configure_agent(self.__agent_id, self.__agent_config.as_json)

        self.action_space = None
        self.observation_space = None
        if self.__agent_id != INVALID_ID:
            response = self.__rpc_client.desc_action_space(self.__agent_id)
            action_space = spaces.gym_space_from_mladapter(response)
            if action_space is not None and (type(action_space) != list or len(action_space) > 0):
                self.action_space = action_space

            response = self.__rpc_client.desc_observation_space(self.__agent_id)
            observation_space = spaces.gym_space_from_mladapter(response)
            if observation_space is not None and (type(observation_space) != list or len(observation_space) > 0):
                self.observation_space = observation_space

    def _new_agent(self, agent_config):
        if agent_config:
            return self.__rpc_client.create_agent(agent_config.as_json)
        return self.__rpc_client.add_agent()
    
    def __del__(self):
        self.close()

    def render(self):
        # do nothing for now, UnrealEngine instance will do the rendering on its own as part of 'step' call
        pass

    def _set_realtime(self, realtime):
        self._realtime = realtime
        
        if not realtime:
            self._world_step_impl = self._tick_world
        elif self._action_duration_seconds != None:
            self._world_step_impl = self.wait_for_action_duration
        else:
            self._world_step_impl = lambda: None

        if hasattr(self.__rpc_client, 'enable_manual_world_tick'):
            self.__rpc_client.enable_manual_world_tick(not realtime)

        if hasattr(self.__rpc_client, 'enable_action_duration') and self._action_duration_seconds != None:
            self.__rpc_client.enable_action_duration(self.__agent_id, True, self._action_duration_seconds)

    def _get_observation(self):
        raw_obs = self.__rpc_client.get_observations(self.__agent_id)
        return gym.spaces.unflatten(self.observation_space, raw_obs)

    def _tick_world(self):
        self.__rpc_client.request_world_tick(self._frames_step, True)
    
    def wait_for_action_duration(self):
        self.__rpc_client.wait_for_action_duration(self.__agent_id)

    def reset(self, wait_action=None, skip_time=1):
        if not self.is_connected():
            raise NotConnected
        logger.debug('{}: reset'.format(self._debug_id))
        self.__rpc_client.reset()
        # wait until game says it's not over        
        while not self.__rpc_client.is_ready():
            if self.action_space is not None:
                self.__rpc_client.act(self.__agent_id, self.wrap_action(
                    self.action_space.sample() if wait_action is None else wait_action))
            self.skip(skip_time)
        self._steps_performed = 0
        return self._get_observation()

    def close(self, shutdown=True):
        """ Disconnects this environment from the simulation.
        Args:
            shutdown (bool): if True will shut down the simulation as well.
        """
        logger.debug('{}: close(shutdown={})'.format(self._debug_id, shutdown))
        if self.is_connected():
            try:
                logger.info('Closing connection on port {}'.format(self.__server_port))
                if not self._realtime:
                    # resume the natural flow of the sim
                    self.__rpc_client.enable_manual_world_tick(False)
                self.__rpc_client.close()
            except msgpackrpc.error.TimeoutError:
                logger.debug(
                    "msgpackrpc.error.TimeoutError occurred. Ignoring due to the environment being closed anyway.")
        self.__rpc_client = None

        if shutdown:
            UERunner.stop(self.__engine_process)
            self.__engine_process = None

    @property
    def _agent_id(self):
        return self.__agent_id

    @property
    def conn(self):
        return self.__rpc_client

    @property
    def game_over(self):
        return self.is_finished()

    def is_finished(self):
        return self.__rpc_client.is_finished(self.__agent_id)

    def skip(self, interval):
        if self._realtime:
            time.sleep(interval)
        else:
            # @todo measure time or add rpc function to skip X seconds
            self.__rpc_client.request_world_tick(int(interval), True)
            self._steps_performed += interval

    def wrap_action(self, action):
        return list(map(float, gym.spaces.flatten(self.action_space, action)))

    def get_reward(self):
        return self.__rpc_client.get_reward(self.__agent_id)

    def step(self, action=None):
        """
        :param action: action that will be passed over to the RPC server. If action is None then 'act' won't get called
        """        
        if self.action_space is not None and action is not None:
            assert self.action_space.contains(action)
            action = self.wrap_action(action)
            self.__rpc_client.act(self.__agent_id, action)
        
        self._steps_performed += 1
        self._world_step_impl()
        observation = self._get_observation()
        reward = self.get_reward()
        return observation, reward, self.game_over, {}

    @staticmethod
    def default_agent_config():
        """
        Override this static function to supply your environment-specific agent config
        """
        return None
    
    
class UnrealEnvMultiAgent(UnrealEnv):
    """
    @Note This environment is not that compatible with openai gym in that the functions return lists of values rather
        than just values 
    """

    def __init__(self, agent_count=1, agent_configs=[None], **kwargs):
    
        assert agent_configs is None or type(agent_configs) == list, 'agent_configs needs to be a list'
        
        self.__agent_configs = agent_configs if (agent_configs is not None and len(agent_configs) != 0) \
            else [kwargs['agent_config'] for _ in range(agent_count)]
        
        super(UnrealEnvMultiAgent, self).__init__(**kwargs)
    
        if type(agent_configs) is not list:
            agent_configs = [agent_configs]
        elif len(agent_configs) == 0:
            agent_configs = [None]

    def reset(self, wait_action=None, skip_time=1):
        logger.debug('{}: reset'.format(self._debug_id))
        self.conn.reset()
        
        obs_n = []
        for agent_id in self.__agent_ids:
        # wait until game says it's not over        
            while not self.conn.is_ready():
                if self.action_space is not None:
                    self.conn.act(agent_id,
                                          self.wrap_action(self.action_space.sample() if wait_action is None else wait_action))
                self.skip(skip_time)
        obs_n.append(self._get_observation(agent_id))
        
        # we've got a single-agent special-case to match openai gym's API   
        return obs_n[0] if len(self.__agent_ids) == 1 else obs_n
    
    
    def step(self, actions=None):
        """
        :param action: action that will be passed over to the RPC server. Note that the value of None is valid only
            if self.action_space is empty (the receiving agent doesn't have any actuators)
            Note2: len(actions) needs to be equal len(self.__agent_ids)
        """
        if actions is not None:
            self.conn.batch_act(self.__agent_ids, actions)
        self._world_step_impl()
        last_observation = self.get_observation()
        reward = self.get_reward()
        return last_observation, reward, self.game_over, {}
    
    
    def get_reward(self):
        return self.conn.batch_get_reward(self.__agent_ids)
    
    
    def get_observation(self):
        raw_obs = self.conn.batch_get_observations(self.__agent_ids)
        return [gym.spaces.unflatten(self.observation_space, ob) for ob in raw_obs]
    
    
    def get_reward(self):
        return self.conn.batch_get_rewards(self.__agent_ids)
    
    
    def is_finished(self):
        """ returns a list of 'is finished' values for all agents"""
        return self.conn.batch_is_finished(self.__agent_ids)
    
    
    def _setup_agents(self, reacquire):
        self.__agent_ids = []
        if reacquire:
            # not the if re reacquiring we're using only the first agent config, as per above assert check 
            current_agent_ids = self.conn.get_all_agent_ids()
            for agent_id, agent_config in zip(current_agent_ids, self.__agent_configs):
                assert agent_id != INVALID_ID
                self.conn.configure_agent(agent_id, agent_config.as_json)               
                self.__agent_ids.append(agent_id)
        
        # create remaining new ones
        reacquired_count = len(self.__agent_ids)
        for i in range(reacquired_count, len(self.__agent_configs)):
            self.__agent_ids.append(self._new_agent(self.__agent_configs[i]))
    
        # figure out the action space is it hasn't been created by child class         
        if self.action_space is None:
            for agent_id in self.__agent_ids:
                response = self.conn.desc_action_space(agent_id)
                action_space = spaces.gym_space_from_mladapter(response)
                if action_space is not None:
                    # we only support all agents using the same action space
                    self.action_space = action_space
                    break
    
        # figure out the observation space is it hasn't been created by child class         
        if self.observation_space is None:
            for agent_id in self.__agent_ids:
                response = self.conn.desc_observation_space(agent_id)
                observation_space = spaces.gym_space_from_mladapter(response)
                if observation_space is not None:
                    # we only support all agents using the same observation space
                    self.observation_space = observation_space
                    break

    def _agent_id(self):
        assert False, 'using agent_id is meaningless in UnrealEnvMultiAgent'
        return INVALID_ID
    