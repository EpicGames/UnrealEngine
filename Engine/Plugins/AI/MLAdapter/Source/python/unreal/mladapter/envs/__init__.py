# Copyright Epic Games, Inc. All Rights Reserved.

import gym
from gym.envs.registration import register
from .platformer import PlatformerGame
from .action_rpg import ActionRPG
from ..runner import UEParams

__all__ = ['PlatformerGame', 'ActionRPG']

fast_kwargs = {
    'ue_params': UEParams(rendering=False, sound=False, single_thread=True)
}

default_kwargs = {
    'ue_params': UEParams(sound=False)
}

real_time_kwargs = {
    'ue_params': UEParams(sound=False),
    'realtime': True
}

# no rendering, no sound, single thread, deterministic with fixed time step
register(
    id='UnrealEngine-ActionRPGFast-v0',
    entry_point='unreal.mladapter.envs:ActionRPG',
    kwargs=fast_kwargs
)

register(
    id='UnrealEngine-ActionRPG-v0',
    entry_point='unreal.mladapter.envs:ActionRPG',
    kwargs=default_kwargs
)

register(
    id='UnrealEngine-ActionRPGRealTime-v0',
    entry_point='unreal.mladapter.envs:ActionRPG',
    kwargs=real_time_kwargs
)

register(
    id='UnrealEngine-PlatformerFast-v0',
    entry_point='unreal.mladapter.envs:PlatformerGame',
    kwargs=fast_kwargs
)

register(
    id='UnrealEngine-Platformer-v0',
    entry_point='unreal.mladapter.envs:PlatformerGame',
    kwargs=default_kwargs
)

register(
    id='UnrealEngine-PlatformerRealTime-v0',
    entry_point='unreal.mladapter.envs:PlatformerGame',
    kwargs=real_time_kwargs
)
