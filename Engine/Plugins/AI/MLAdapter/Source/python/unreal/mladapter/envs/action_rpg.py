# Copyright Epic Games, Inc. All Rights Reserved.

from ..core import UnrealEnv, AgentConfig
from ..utils import dict_from_json


class ActionRPG(UnrealEnv):
    
    PROJECT_NAME = 'ActionRPG'
    
    def __init__(self, ue_params=None, **kwargs):
        self.health_sensor_index = None
        self._last_obs = None

        if ue_params is not None:
            ue_params.set_default_map_name('ActionRPG_P')
            
        super().__init__(ue_params=ue_params, **kwargs)

        response = self.conn.desc_observation_space(self._agent_id)
        if type(response) == str or type(response) == bytes:
            response = dict_from_json(response)

        observations_desc = dict_from_json(response)
        self.health_sensor_index = list(observations_desc.keys()).index('Attribute')  
        
    def _get_observation(self):
        self._last_obs = super()._get_observation()
        return self._last_obs
    
    def reset(self, wait_action=None, skip_time=1):
        # custom skip before resetting to give the game's BP control flow time to 'rest'
        self.skip(5)    
        ret = super().reset(wait_action, skip_time)
        return ret

    def get_reward(self): 
        return self._steps_performed

    @staticmethod
    def default_agent_config():
        agent_config = AgentConfig()
        agent_config.add_sensor('AIPerception',
                                {'mode': 'rotator', 'count': 3, 'sort': 'distance', 'peripheral_angle': 180})
        agent_config.add_sensor('Attribute', {'attributes': 'health, mana'})
        agent_config.add_actuator('InputKey', {'ignore_keys': 'esc, backspace', 'ignore_actions': 'pause, inventory'})
        agent_config.avatarClassName = 'bp_playercharacter_c'
        return agent_config
