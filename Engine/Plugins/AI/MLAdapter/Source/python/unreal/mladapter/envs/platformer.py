# Copyright Epic Games, Inc. All Rights Reserved.

from ..core import UnrealEnv, AgentConfig
import numpy as np


class PlatformerGame(UnrealEnv):
    
    PROJECT_NAME = 'PlatformerGame'
    
    """
    @Note due to how PlatformerGame's in-game menu is implemented a command line parameter is required to disable it.
    When PlatformerGame environment is responsible for launching the game it will all the parameter automatically (see
    the ctor).
    """
    def __init__(self, ue_params=None, **kwargs):

        if ue_params is not None:
            ue_params.set_default_map_name('Platformer_StreetSection')

        if 'timeout' not in kwargs:
            kwargs['timeout'] = 120
        super(PlatformerGame, self).__init__(ue_params=ue_params, **kwargs)
        
        # touch up the observation space with appropriate observation range
        self.obs_low = np.array([-1.5250578e+04, 2.6320000e+03, -1.2890797e+02, -2.0756834e+01, -5.6499330e-04, -2.6066665e+03],
             self.observation_space.dtype)
        shape = self.obs_low.shape
        self.observation_space.low = np.zeros(shape)

        self.obs_high = np.array([4.6085508e+04, 2.6369165e+03, 3.1607444e+03, 1.5097178e+03, 1.7980594e-04, 2.1533333e+03],
             self.observation_space.dtype)
        self.observation_space.high = np.ones(shape)

        # we need to modify the action space to make the agent not use 'Esc' key
        self.action_space.n -= 1 

        self.obs_span = self.obs_high - self.obs_low
        self._prev_projected_finish = 0

        self._last_observations = []

    def wrap_action(self, action):
        """
        got to add a '0' at the end to account for all the actions the server is expecting
        (we did remove 1 element in the constructor)
        """
        return super().wrap_action(action) + [0.0]
        
    def _get_observation(self):
        #raw_obs = self.__rpc_client.get_observations(self._agent_id)
        #return gym.spaces.unflatten(self.observation_space, raw_obs)
        obs = super()._get_observation()
        obs = (obs - self.obs_low) / self.obs_span
        self._last_observations = obs 
        return obs
    
    def reset(self, wait_action=None):
        while not self.conn.is_ready():
            self.conn.act(self._agent_id,
                          self.wrap_action(self.action_space.sample() if wait_action is None else wait_action))
            self.skip(1)

        self.conn.reset()

        while not self.conn.is_ready():
            self.conn.act(self._agent_id,
                          self.wrap_action(self.action_space.sample() if wait_action is None else wait_action))
            self.skip(1)
            
        return self._get_observation()
    
    def get_reward(self):
        """
        Raw 'get_reward' call to PlatformerGame retrieves current run time. The function is transforming the reward
        to be related to the projected finish time with current progress on the track
        """
        threshold = 0.001
        pct_traveled = max(min(self._last_observations[0], 1.0), 0.0)
        current_time = super().get_reward()
        projected_finish = current_time / max(threshold, pct_traveled)
        
        if pct_traveled < threshold:
            reward = 0
        else:
            reward = min((-(projected_finish - self._prev_projected_finish)), 1)

        self._prev_projected_finish = projected_finish
        return reward  

    @staticmethod
    def default_agent_config():
        agent_config = AgentConfig()
        agent_config.add_actuator('InputKey')
        agent_config.add_sensor('Movement', {'location': 'absolute'})
        return agent_config
