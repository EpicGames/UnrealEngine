# Copyright Epic Games, Inc. All Rights Reserved.

from gym.error import Error
import msgpackrpc


class MLAdapterError(Exception):
    pass


class FailedToLaunch(MLAdapterError):
    """Raised when the we fail to launch the executable, usually due to wrong executable directory or file being used.
    """
    pass


class UnrealEngineInstanceNoLongerRunning(MLAdapterError):
    """Raised when we detect that the UnrealEngine instance launched by the script is no longer running. It could have crashed,
        been closed manually or the instance itself has shut down. Check UnrealEngine instance's logs for details.
    """
    pass


class UnableToReachRPCServer(MLAdapterError):
    """Raised when initial calls to rpc server fail (due to time out or reconnection limit reaching)
    """
    pass


class NotConnected(MLAdapterError):
    """Raised when the we fail to launch the executable, usually due to wrong executable directory or file being used.
    """
    pass


# error aliases
RPCError = msgpackrpc.error.RPCError
ReconnectionLimitReached = msgpackrpc.error.TransportError
ConnectionTimeoutError = msgpackrpc.error.TimeoutError
