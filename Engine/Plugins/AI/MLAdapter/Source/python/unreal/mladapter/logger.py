# Copyright Epic Games, Inc. All Rights Reserved.

from gym import logger


set_level = logger.set_level
DEBUG = logger.DEBUG
INFO = logger.INFO
WARN = logger.WARN
ERROR = logger.ERROR
DISABLED = logger.DISABLED


def debug(msg, *args):
    logger.debug('unreal.mladapter: ' + msg, *args)

def info(msg, *args):
    logger.info('unreal.mladapter: ' + msg, *args)

def warn(msg, *args):
    logger.warn('unreal.mladapter: ' + msg, *args)

def error(msg, *args):
    logger.error('unreal.mladapter: ' + msg, *args)

def get_level():
    return logger.MIN_LEVEL
