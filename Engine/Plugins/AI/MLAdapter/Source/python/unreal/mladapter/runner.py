# Copyright Epic Games, Inc. All Rights Reserved.

from subprocess import Popen
from . import logger
import copy
import os
import argparse
from .error import *

DEFAULT_FPS = 20

UEDEVBINKEY = 'UE-DevBinaries'
EDITOR = True

EXEC_NAME = None
__DEBUG = None
_EXECUTABLE_OVERRIDE = None



def set_debug(debug, config=''):
    global EXEC_NAME
    global __DEBUG
    __DEBUG = debug
    if __DEBUG:
        EXEC_NAME = f'UnrealEditor-Win64-Debug{config}.exe'
    else:
        EXEC_NAME = 'UnrealEditor.exe'


set_debug(False)


def is_debug():
    global __DEBUG
    return __DEBUG


def set_executable(exec_name):
    global _EXECUTABLE_OVERRIDE
    _EXECUTABLE_OVERRIDE = exec_name


def _get_exec_path():
    binaries_path = '.'
    if UEDEVBINKEY not in os.environ:
        if _EXECUTABLE_OVERRIDE is None:
            logger.warn("'{}' environment variable couldn't be found, falling back to current directory as binaries \
directory. Set '{}' environment variable or use '--exec' command line parameter".format(UEDEVBINKEY, UEDEVBINKEY))
    else:
        binaries_path = os.environ[UEDEVBINKEY]
    return os.path.join(binaries_path, EXEC_NAME) if _EXECUTABLE_OVERRIDE is None else _EXECUTABLE_OVERRIDE


class UEParams(object):
    WINDOW = 'windowed'
    FIXED_TIME_STEP = 'usefixedtimestep'
    GAME = 'game'
    UNATTENDED = 'unattended'
    
    def __init__(self, map_name='', rendering=True, sound=True, single_thread=False, fixed_time_step=False, custom=''):
        self.__options = [UEParams.WINDOW, UEParams.GAME, UEParams.UNATTENDED]
        self.__params = {'resx': 320, 'resy': 240, 'fps': DEFAULT_FPS}
        self.map_name = map_name
        self.custom = custom
        self.fixed_time_step(fixed_time_step)
        self.rendering(rendering)
        self.sound(sound)
        self.single_thread(single_thread)

    @classmethod
    def from_args(cls, args):
        params = cls()
        if args is not None:
            params.rendering(not hasattr(args, 'norendering') or args.norendering == 0)
            params.single_thread(hasattr(args, 'nothreads') and args.nothreads == 1)
            params.sound(not hasattr(args, 'nosound') or args.nosound == 0)
            if hasattr(args, 'resx') and hasattr(args, 'resy'): 
                params.resolution(args.resx, args.resy)
        return params

    def set_default_map_name(self, name):
        self.map_name = name

    def resolution(self, x, y):
        self.__params['resx'] = x
        self.__params['resy'] = y
        
    def rendering(self, enabled):
        self.set_enable_option('nullrhi', not enabled)

    def single_thread(self, enabled):
        self.set_enable_option('onethread', enabled)
        self.set_enable_option('ReduceThreadUsage', enabled)

    def fixed_time_step(self, enabled):
        self.set_enable_option(UEParams.FIXED_TIME_STEP, enabled)

    def sound(self, enabled):
        self.set_enable_option('nosound', not enabled)

    def deterministic(self, enabled):
        self.set_enable_option('deterministic', enabled)        
        
    def as_command_line(self):
        cmdline = self.map_name
        for o in self.__options:
            cmdline += ' -{}'.format(o)
        for k, v in self.__params.items():
            cmdline += ' -{}={}'.format(k, v)
        return cmdline + ' ' + self.custom

    def set_enable_option(self, option, enabled):
        if enabled:
            if option not in self.__options:
                self.__options.append(option)
        elif option in self.__options:
            self.__options.remove(option)
            
    def add_option(self, options):
        if type(options) is not list:
            options = [options]
        for o in options:
            self.set_enable_option(o, True)

    def add_param(self, param, value):
        self.__params[param] = value

    def remove_param(self, param):
        if param in self.__params:
            del self.__params[param]
        
    def copy(self):
        return copy.deepcopy(self)


class UERunner:
    @staticmethod
    def run(project_name, ue_params, server_port):
        ue_params.add_param('MLAdapterPort', '{}'.format(server_port))
        if is_debug():
            ue_params.add_option('debug')
        commandline = '{} {} {}'.format(_get_exec_path(), project_name, ue_params.as_command_line())
        logger.info('Launching UnrealEngine instance with commandline:\n\t {}'.format(commandline))
        try:
            return Popen(commandline)
        except (FileNotFoundError, OSError):
            logger.error('Failed to launch the executable. Please verify the executable\'s path')
            raise FailedToLaunch

    @staticmethod
    def stop(engine_subprocess: Popen):
        if engine_subprocess is not None:
            engine_subprocess.terminate()
