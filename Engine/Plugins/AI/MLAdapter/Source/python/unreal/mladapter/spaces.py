# Copyright Epic Games, Inc. All Rights Reserved.

from .utils import dict_from_json
import gym.spaces
from collections import OrderedDict


def create_space(type_name, params):
    assert type(type_name) == str
    if type_name == 'Discrete':
        return gym.spaces.Discrete(params)
    elif type_name == 'MultiDiscrete':
        return gym.spaces.MultiDiscrete(params)
    elif type_name == 'Box':
        assert len(params) >= 3
        low = params[0]
        high = params[1]
        shape = params[2:]
        return gym.spaces.Box(low, high, shape=shape)
    elif type_name == 'Tuple':
        return gym_space_from_mladapter(params)
    else:
        return gym.spaces.Dict({type_name: gym_space_from_mladapter(params)})


def gym_space_from_mladapter(data):
    if type(data) == str or type(data) == bytes:    
        data = dict_from_json(data)
    return gym_space_from_list(data) if data is not None else None

    
def gym_space_from_list(data):
    # when called with an OrderedDict we only are about the values. Keys are there for debugging and readers'
    # convenience reasons
    if type(data) == OrderedDict:
        data = list(data.values())

    if type(data) != list:
        raise TypeError('Only OrderedDict and list are supported while data is of type {}'.format(type(data)))
        return None
    
    if len(data) == 0:
        return None

    spaces = []
    for d in data:
        if type(d) == OrderedDict:
            for k, v in d.items():
                spaces.append(create_space(k, v))
        else:
            spaces.append(gym_space_from_mladapter(d))
    
    if len(spaces) == 1:
        return spaces[0]
    else:
        return gym.spaces.Tuple(tuple(spaces))
