# Copyright Epic Games, Inc. All Rights Reserved.

import json
import msgpackrpc
import argparse
from collections import OrderedDict

LOCALHOST = '127.0.0.1'
DEFAULT_PORT = 15151
DEFAULT_TIMEOUT = 120


def dict_from_json(data):
    if type(data) == bytes:
        data = data.decode('utf8')
    if type(data) == str:
        if not data:
            data = None
        else:
            # using OrderedDict to retain items order 
            data = json.loads(data, object_pairs_hook=OrderedDict)
            if type(data) == OrderedDict:
                for key in data:
                    # at this point data[key] is a string and we need the structure represented by it
                    data[key] = dict_from_json(data[key])
    return data


def is_port_available(port, host=LOCALHOST):
    import socket    
    with socket.socket(socket.AF_INET, socket.SOCK_STREAM) as s:
        return s.connect_ex((host, port)) != 0


def find_available_port(host=LOCALHOST):
    import socket
    with socket.socket(socket.AF_INET, socket.SOCK_STREAM) as s:
        s.bind((host, 0))
        s.listen(1)
        return s.getsockname()[1]
    

def random_action(env):
    return env.action_space.sample() if env.action_space else None


class ArgumentParser(argparse.ArgumentParser):
    def __init__(self, **kwargs):
        super().__init__(kwargs)
        self.add_argument('--env', type=str, default='UnrealEngine-ActionRPG-v0', help='environment ID')
        self.add_argument('--nothreads', type=int, default=0)
        self.add_argument('--norendering', type=int, default=0)
        self.add_argument('--nosound', type=int, default=0)
        self.add_argument('--resx', type=int, default=800)
        self.add_argument('--resy', type=int, default=600)
        self.add_argument('--exec', type=str, default=None)
        self.add_argument('--port', type=int, default=DEFAULT_PORT)

    def parse_args(self, **kwargs):
        args = super().parse_args(**kwargs)

        if args.exec is not None:
            from unreal.mladapter.runner import set_executable
            set_executable(args.exec)

        return args
