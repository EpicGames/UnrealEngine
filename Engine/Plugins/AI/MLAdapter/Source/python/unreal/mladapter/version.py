# Copyright Epic Games, Inc. All Rights Reserved.

VERSION = '0.1.2'
